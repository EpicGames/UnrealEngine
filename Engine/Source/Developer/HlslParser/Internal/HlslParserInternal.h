// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/Function.h"

namespace UE::HlslParser
{

// Structure to hold forward declarations for a specific scope/namespace chain for the HlslParser
struct FScopedDeclarations
{
	FScopedDeclarations(TConstArrayView<FStringView> InScope, TConstArrayView<FStringView> InSymbols)
		: Scope(InScope)
		, Symbols(InSymbols)
	{
	}
	TConstArrayView<FStringView> Scope;
	TConstArrayView<FStringView> Symbols;
};

extern HLSLPARSER_API bool RemoveUnusedOutputs(
	FString& InOutSourceCode,
	TConstArrayView<FStringView> InUsedOutputs,
	TConstArrayView<FStringView> InExceptions,
	TConstArrayView<FScopedDeclarations> InScopedDeclarations,
	FString& InOutEntryPoint,
	TArray<FString>& OutErrors
);

extern HLSLPARSER_API bool RemoveUnusedOutputs(FString& InOutSourceCode, const TArray<FString>& InUsedOutputs, const TArray<FString>& InExceptions, FString& InOutEntryPoint, TArray<FString>& OutErrors);

extern HLSLPARSER_API bool RemoveUnusedInputs(
	FString& InOutSourceCode,
	TConstArrayView<FStringView> InUsedInputs,
	TConstArrayView<FScopedDeclarations> InScopedDeclarations,
	FString& InOutEntryPoint,
	TArray<FString>& OutErrors
);

extern HLSLPARSER_API bool RemoveUnusedInputs(FString& InOutSourceCode, const TArray<FString>& InUsedInputs, FString& InOutEntryPoint, TArray<FString>& OutErrors);

// Shader input/output parameter storage classes. Naming adopted from SPIR-V nomenclature.
enum class EShaderParameterStorageClass
{
	Input,
	Output,
};

// Returns the semantic names of all individual entry point parameters (i.e. all structure fields are inlined)
extern HLSLPARSER_API bool FindEntryPointParameters(
	const FString& InSourceCode,
	const FString& InEntryPoint,
	EShaderParameterStorageClass ParameterStorageClass,
	TConstArrayView<FScopedDeclarations> InScopedDeclarations,
	TArray<FString>& OutParameterSemantics,
	TArray<FString>& OutErrors
);

// Invokes the input callback for each identifier in the specified HLSL source code until the callback returns false.
extern HLSLPARSER_API bool ForEachHlslIdentifier(const FString& InSourceCode, const FString& InFilename, const TFunction<bool(FStringView)>& InIdentifierCallback);

} // namespace UE::HlslParser
