// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HlslAST.cpp - Abstract Syntax Tree implementation for HLSL.
=============================================================================*/

#include "HlslAST.h"

namespace CrossCompiler
{
	namespace AST
	{
		static void WriteOptionArraySize(FASTWriter& Writer, bool bIsArray, const TLinearArray<FExpression*>& ArraySize)
		{
			if (bIsArray && ArraySize.Num() == 0)
			{
				Writer << TEXT("[]");
			}
			else
			{
				for (const FExpression* Dimension : ArraySize)
				{
					Writer << TEXT('[');
					if (Dimension)
					{
						Dimension->Write(Writer);
					}
					Writer << TEXT(']');
				}
			}
		}

		FNode::FNode(FLinearAllocator* Allocator, const FSourceInfo& InInfo) :
			SourceInfo(InInfo),
			Attributes(Allocator)/*,
			Prev(nullptr),
			Next(nullptr)*/
		{
		}

		void FASTWriter::DoIndent()
		{
			for (int32 TabIndex = 0; TabIndex < Indent; ++TabIndex)
			{
				(*this) << TEXT('\t');
			}
		}

		void FNode::WriteAttributes(FASTWriter& Writer) const
		{
			if (Attributes.Num() > 0)
			{
				for (auto* Attr : Attributes)
				{
					Attr->Write(Writer);
				}

				Writer << TEXT(' ');
			}
		}

		FPragma::FPragma(FLinearAllocator* InAllocator, const TCHAR* InPragma, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo)
		{
			Pragma = InAllocator->Strdup(InPragma);
		}

		void FPragma::Write(FASTWriter& Writer) const
		{
			Writer << Pragma << TEXT("\n");
		}

		FExpression::FExpression(FLinearAllocator* InAllocator, EOperators InOperator, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			Operator(InOperator),
			Identifier(nullptr),
			Expressions(InAllocator)
		{
			TypeSpecifier = nullptr;
		}

		FExpression::FExpression(FLinearAllocator* InAllocator, EOperators InOperator, FExpression* E0, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			Operator(InOperator),
			Identifier(nullptr),
			Expressions(InAllocator)
		{
			Expressions.SetNumUninitialized(1);
			Expressions[0] = E0;
			TypeSpecifier = nullptr;
		}

		FExpression::FExpression(FLinearAllocator* InAllocator, EOperators InOperator, FExpression* E0, FExpression* E1, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			Operator(InOperator),
			Identifier(nullptr),
			Expressions(InAllocator)
		{
			Expressions.SetNumUninitialized(2);
			Expressions[0] = E0;
			Expressions[1] = E1;
			TypeSpecifier = nullptr;
		}

		FExpression::FExpression(FLinearAllocator* InAllocator, EOperators InOperator, FExpression* E0, FExpression* E1, FExpression* E2, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			Operator(InOperator),
			Identifier(nullptr),
			Expressions(InAllocator)
		{
			Expressions.SetNumUninitialized(3);
			Expressions[0] = E0;
			Expressions[1] = E1;
			Expressions[2] = E2;
			TypeSpecifier = nullptr;
		}

		void FExpression::WriteOperator(FASTWriter& Writer) const
		{
			switch (Operator)
			{
			case EOperators::Plus:
				Writer << TEXT("+");
				break;

			case EOperators::Minus:
				Writer << TEXT("-");
				break;

			case EOperators::Assign:
				Writer << TEXT("=");
				break;

			case EOperators::AddAssign:
				Writer << TEXT("+=");
				break;

			case EOperators::SubAssign:
				Writer << TEXT("-=");
				break;

			case EOperators::MulAssign:
				Writer << TEXT("*=");
				break;

			case EOperators::DivAssign:
				Writer << TEXT("/=");
				break;

			case EOperators::ModAssign:
				Writer << TEXT("%=");
				break;

			case EOperators::RSAssign:
				Writer << TEXT(">>=");
				break;

			case EOperators::LSAssign:
				Writer << TEXT("<<=");
				break;

			case EOperators::AndAssign:
				Writer << TEXT("&=");
				break;

			case EOperators::OrAssign:
				Writer << TEXT("|=");
				break;

			case EOperators::XorAssign:
				Writer << TEXT("^=");
				break;

			case EOperators::Conditional:
				Writer << TEXT("?");
				break;

			case EOperators::LogicOr:
				Writer << TEXT("||");
				break;

			case EOperators::LogicAnd:
				Writer << TEXT("&&");
				break;

			case EOperators::LogicNot:
				Writer << TEXT("!");
				break;

			case EOperators::BitOr:
				Writer << TEXT("|");
				break;

			case EOperators::BitXor:
				Writer << TEXT("^");
				break;

			case EOperators::BitAnd:
				Writer << TEXT("&");
				break;

			case EOperators::BitNeg:
				Writer << TEXT("~");
				break;

			case EOperators::Equal:
				Writer << TEXT("==");
				break;

			case EOperators::NEqual:
				Writer << TEXT("!=");
				break;

			case EOperators::Less:
				Writer << TEXT("<");
				break;

			case EOperators::Greater:
				Writer << TEXT(">");
				break;

			case EOperators::LEqual:
				Writer << TEXT("<=");
				break;

			case EOperators::GEqual:
				Writer << TEXT(">=");
				break;

			case EOperators::LShift:
				Writer << TEXT("<<");
				break;

			case EOperators::RShift:
				Writer << TEXT(">>");
				break;

			case EOperators::Add:
				Writer << TEXT("+");
				break;

			case EOperators::Sub:
				Writer << TEXT("-");
				break;

			case EOperators::Mul:
				Writer << TEXT("*");
				break;

			case EOperators::Div:
				Writer << TEXT("/");
				break;

			case EOperators::Mod:
				Writer << TEXT("%");
				break;

			case EOperators::PreInc:
				Writer << TEXT("++");
				break;

			case EOperators::PreDec:
				Writer << TEXT("--");
				break;

			case EOperators::PostInc:
			case EOperators::PostDec:
			case EOperators::FieldSelection:
			case EOperators::ArrayIndex:
				break;

			case EOperators::TypeCast:
				Writer << TEXT('(');
				TypeSpecifier->Write(Writer);
				Writer << TEXT(')');
				break;

			case EOperators::SizeOf:
				Writer << TEXT("sizeof");
				break;

			default:
				Writer << TEXT("*MISSING_");
				Writer << (uint32)Operator;
				Writer << TEXT('*');
				checkf(0, TEXT("Unhandled AST Operator %d!"), (uint32)Operator);
				break;
			}
		}

		void FExpression::Write(FASTWriter& Writer) const
		{
			switch (Operator)
			{
			case EOperators::Conditional:
				Writer << TEXT('(');
				Expressions[0]->Write(Writer);
				Writer << TEXT(" ? ");
				Expressions[1]->Write(Writer);
				Writer << TEXT(" : ");
				Expressions[2]->Write(Writer);
				Writer << TEXT(")");
				break;

			case EOperators::Literal:
				Writer << Identifier;
				break;

			case EOperators::Identifier:
				Writer << Identifier;
				break;

			default:
				Writer << TEXT("*MISSING_");
				Writer << (uint32)Operator;
				Writer << TEXT('*');
				checkf(0, TEXT("Unhandled AST Operator %d!"), (int32)Operator);
				break;
			}
		}

		bool FExpression::GetConstantIntValue(int32& OutValue) const
		{
			if (IsConstant())
			{
				checkf(Identifier!=nullptr, TEXT("Null identifier, literaltype %d"), (int32)LiteralType);
				OutValue = (int32)FCString::Atoi(Identifier->GetData());
				return true;
			}

			return false;
		}

		FExpression::~FExpression()
		{
			for (FExpression* Expr : Expressions)
			{
				delete Expr;
			}
		}

		FUnaryExpression::FUnaryExpression(FLinearAllocator* InAllocator, EOperators InOperator, FExpression* Expr, const FSourceInfo& InInfo) :
			FExpression(InAllocator, InOperator, Expr, InInfo)
		{
		}

		void FUnaryExpression::Write(FASTWriter& Writer) const
		{
			WriteOperator(Writer);

			if (Writer.ExpressionScope != 0 && Operator != EOperators::FieldSelection)
			{
				Writer << TEXT('(');
			}

			if (Expressions.Num() != 0)
			{
				++Writer.ExpressionScope;
				Expressions[0]->Write(Writer);
				--Writer.ExpressionScope;
			}

			// Suffix
			switch (Operator)
			{
			case EOperators::PostInc:
				Writer << TEXT("++");
				break;

			case EOperators::PostDec:
				Writer << TEXT("--");
				break;

			case EOperators::FieldSelection:
				Writer << TEXT('.');
				Writer << Identifier;
				break;

			default:
				break;
			}

			if (Writer.ExpressionScope != 0 && Operator != EOperators::FieldSelection)
			{
				Writer << TEXT(')');
			}
		}

		FBinaryExpression::FBinaryExpression(FLinearAllocator* InAllocator, EOperators InOperator, FExpression* E0, FExpression* E1, const FSourceInfo& InInfo) :
			FExpression(InAllocator, InOperator, E0, E1, InInfo)
		{
		}

		void FBinaryExpression::Write(FASTWriter& Writer) const
		{
			switch (Operator)
			{
			case EOperators::ArrayIndex:
				if (Expressions[0]->AsUnaryExpression() && Expressions[0]->Operator == EOperators::Identifier)
				{
					Expressions[0]->Write(Writer);
				}
				else
				{
					Writer << TEXT('(');
					Expressions[0]->Write(Writer);
					Writer << TEXT(')');
				}
				Writer << TEXT('[');
				Expressions[1]->Write(Writer);
				Writer << TEXT(']');
				break;

			default:
				if (Writer.ExpressionScope != 0 && !IsAssignmentOperator(Operator))
				{
					Writer << TEXT('(');
				}
				++Writer.ExpressionScope;
				Expressions[0]->Write(Writer);
				Writer << TEXT(' ');
				WriteOperator(Writer);
				Writer << TEXT(' ');
				Expressions[1]->Write(Writer);
				--Writer.ExpressionScope;
				if (Writer.ExpressionScope != 0 && !IsAssignmentOperator(Operator))
				{
					Writer << TEXT(')');
				}
				break;
			}
		}

		bool FBinaryExpression::GetConstantIntValue(int32& OutValue) const
		{
			int32 LHS = 0;
			int32 RHS = 0;
			if (!Expressions[0]->GetConstantIntValue(LHS) || !Expressions[1]->GetConstantIntValue(RHS))
			{
				return false;
			}

			switch (Operator)
			{
			default:
				return false;

			case EOperators::LogicOr:	OutValue = LHS != 0 || RHS != 0; break;
			case EOperators::LogicAnd:	OutValue = LHS != 0 && RHS != 0; break;
			case EOperators::BitOr:		OutValue = LHS | RHS; break;
			case EOperators::BitXor:	OutValue = LHS ^ RHS; break;
			case EOperators::BitAnd:	OutValue = LHS & RHS; break;
			case EOperators::Equal:		OutValue = LHS == RHS; break;
			case EOperators::NEqual:	OutValue = LHS != RHS; break;
			case EOperators::Less:		OutValue = LHS < RHS; break;
			case EOperators::Greater:	OutValue = LHS > RHS; break;
			case EOperators::LEqual:	OutValue = LHS <= RHS; break;
			case EOperators::GEqual:	OutValue = LHS >= RHS; break;
			case EOperators::LShift:	OutValue = LHS << RHS; break;
			case EOperators::RShift:	OutValue = LHS >> RHS; break;
			case EOperators::Add:		OutValue = LHS + RHS; break;
			case EOperators::Sub:		OutValue = LHS - RHS; break;
			case EOperators::Mul:		OutValue = LHS * RHS; break;
			case EOperators::Div:		OutValue = LHS / RHS; break;
			case EOperators::Mod:		OutValue = LHS % RHS; break;
			}
			return true;
		}

		FExpressionStatement::FExpressionStatement(FLinearAllocator* InAllocator, FExpression* InExpr, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			Expression(InExpr)
		{
		}

		void FExpressionStatement::Write(FASTWriter& Writer) const
		{
			Writer.DoIndent();
			Expression->Write(Writer);
			Writer << TEXT(";\n");
		}

		FExpressionStatement::~FExpressionStatement()
		{
			delete Expression;
		}

		FCompoundStatement::FCompoundStatement(FLinearAllocator* InAllocator, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			Statements(InAllocator)
		{
		}

		void FCompoundStatement::Write(FASTWriter& Writer) const
		{
			Writer.DoIndent();
			Writer << TEXT("{\n");
			for (FNode* Statement : Statements)
			{
				FASTWriterIncrementScope Scope(Writer);
				Statement->Write(Writer);
			}
			Writer.DoIndent();
			Writer << TEXT("}\n");
		}

		FCompoundStatement::~FCompoundStatement()
		{
			for (FNode* Statement : Statements)
			{
				delete Statement;
			}
		}

		FStaticAssertStatement::FStaticAssertStatement(FLinearAllocator* InAllocator, const FSourceInfo& InInfo, const FString& InKeyword) :
			FNode(InAllocator, InInfo),
			Keyword(InKeyword),
			Condition(nullptr)
		{
		}

		FStaticAssertStatement::~FStaticAssertStatement()
		{
			delete Condition;
		}

		void FStaticAssertStatement::Write(FASTWriter& Writer) const
		{
			Writer.DoIndent();
			Writer << *Keyword << TEXT("(");
			checkf(Condition != nullptr, TEXT("Cannot write %s()-statement in HLSL without a conditional expression"), *Keyword);
			Condition->Write(Writer);
			Writer << TEXT(", \"") << *Message << TEXT("\");\n");
		}

		FC99PragmaStatement::FC99PragmaStatement(FLinearAllocator* InAllocator, const FSourceInfo& InInfo, const FString& InArgument) :
			FNode(InAllocator, InInfo),
			Argument(InArgument)
		{
		}

		FC99PragmaStatement::~FC99PragmaStatement()
		{
		}

		void FC99PragmaStatement::Write(FASTWriter& Writer) const
		{
			Writer << TEXT("_Pragma(\"") << *Argument << TEXT("\")\n");
		}

		FFunctionDefinition::FFunctionDefinition(FLinearAllocator* InAllocator, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			Prototype(nullptr),
			Body(nullptr)
		{
		}

		void FFunctionDefinition::Write(FASTWriter& Writer) const
		{
			WriteAttributes(Writer);
			Prototype->Write(Writer);
			if (Body)
			{
				Body->Write(Writer);
			}
		}

		FFunctionDefinition::~FFunctionDefinition()
		{
			delete Prototype;
			delete Body;
		}

		FFunction::FFunction(FLinearAllocator* InAllocator, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			ReturnType(nullptr),
			ScopeIdentifier(nullptr),
			Identifier(nullptr),
			ReturnSemantic(nullptr),
			Parameters(InAllocator),
			bIsDefinition(false),
			bIsOperator(false),
			bIsStatic(false)
		{
		}

		void FFunction::Write(FASTWriter& Writer) const
		{
			WriteAttributes(Writer);
			Writer << TEXT("\n");
			if (bIsStatic)
			{
				Writer << TEXT("static ");
			}
			ReturnType->Write(Writer);
			Writer << (TCHAR)' ';
			if (ScopeIdentifier)
			{
				Writer << ScopeIdentifier << TEXT("::");
			}
			if (bIsOperator)
			{
				Writer << TEXT("operator");
			}
			Writer << Identifier;
			Writer << (TCHAR)'(';
			bool bFirst = true;
			const int32 ParamsPerLine = 6;
			for (int32 Index = 0; Index < Parameters.Num(); ++Index)
			{
				if (Index > 0)
				{
					if ((Index % ParamsPerLine) == 0)
					{
						Writer << TEXT(",\n\t\t");
					}
					else
					{
						Writer << TEXT(", ");
					}
				}
				Parameters[Index]->Write(Writer);
			}

			Writer << TEXT(")");
			if (ReturnSemantic)
			{
				ReturnSemantic->Write(Writer);
			}
			if (bIsDefinition)
			{
				Writer << TEXT(";\n");
			}
			else
			{
				Writer << TEXT("\n");
			}
		}

		FFunction::~FFunction()
		{
			for (FNode* Param : Parameters)
			{
				delete Param;
			}
		}

		FJumpStatement::FJumpStatement(FLinearAllocator* InAllocator, EJumpType InType, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			Type(InType),
			OptionalExpression(nullptr)
		{
		}

		void FJumpStatement::Write(FASTWriter& Writer) const
		{
			Writer.DoIndent();

			switch (Type)
			{
			case EJumpType::Return:
				Writer << TEXT("return");
				break;

			case EJumpType::Break:
				Writer << TEXT("break");
				break;

			case EJumpType::Continue:
				Writer << TEXT("continue");
				break;

			default:
				Writer << TEXT("*MISSING_");
				Writer << (uint32)Type;
				Writer << (TCHAR)'*';
				checkf(0, TEXT("Unhandled AST jump type %d!"), (int32)Type);
				break;
			}

			if (OptionalExpression)
			{
				Writer << TEXT(" ");
				OptionalExpression->Write(Writer);
			}
			Writer << TEXT(";\n");
		}

		FJumpStatement::~FJumpStatement()
		{
			delete OptionalExpression;
		}

		FSelectionStatement::FSelectionStatement(FLinearAllocator* InAllocator, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			Condition(nullptr),
			ThenStatement(nullptr),
			ElseStatement(nullptr)
		{
		}

		void FSelectionStatement::Write(FASTWriter& Writer) const
		{
			Writer.DoIndent();
			WriteAttributes(Writer);
			Writer << TEXT("if (");
			Condition->Write(Writer);
			Writer << TEXT(")\n");
			ThenStatement->Write(Writer);
			if (ElseStatement)
			{
				Writer.DoIndent();
				Writer << TEXT("else\n");
				ElseStatement->Write(Writer);
			}
		}

		FSelectionStatement::~FSelectionStatement()
		{
			delete Condition;
			delete ThenStatement;
			delete ElseStatement;
		}

		FTypeSpecifier::FTypeSpecifier(FLinearAllocator* InAllocator, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			TypeName(nullptr),
			InnerType(nullptr),
			Structure(nullptr),
			TextureMSNumSamples(1),
			PatchSize(0),
			bIsArray(false),
			ArraySize(nullptr)
		{
		}

		void FTypeSpecifier::Write(FASTWriter& Writer) const
		{
			if (Structure)
			{
				Structure->Write(Writer);
			}
			else
			{
				if (bPrecise)
				{
					Writer << TEXT("precise ");
				}
				Writer << TypeName;
				if (TextureMSNumSamples > 1)
				{
					Writer << TEXT('<');
					Writer << InnerType;
					Writer << TEXT(", ");
					Writer << (uint32)TextureMSNumSamples;
					Writer << TEXT('>');
				}
				else if (InnerType && *InnerType)
				{
					Writer << TEXT('<');
					Writer << InnerType;
					Writer << TEXT('>');
				}
			}

			if (bIsArray)
			{
				Writer << TEXT('[');

				if (ArraySize)
				{
					ArraySize->Write(Writer);
				}

				Writer << TEXT(']');
			}
		}

		FTypeSpecifier::~FTypeSpecifier()
		{
			delete Structure;
			delete ArraySize;
		}

		FCBufferDeclaration::FCBufferDeclaration(FLinearAllocator* InAllocator, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			Name(nullptr),
			Declarations(InAllocator)
		{
		}

		void FCBufferDeclaration::Write(FASTWriter& Writer) const
		{
			Writer.DoIndent();
			Writer << TEXT("cbuffer ");
			Writer << Name;
			Writer << TEXT('\n');

			Writer.DoIndent();
			Writer << TEXT("{\n");

			for (FNode* Declaration : Declarations)
			{
				FASTWriterIncrementScope Scope(Writer);
				Declaration->Write(Writer);
			}

			Writer.DoIndent();
			Writer << TEXT("}\n\n");
		}

		FCBufferDeclaration::~FCBufferDeclaration()
		{
			for (FNode* Decl : Declarations)
			{
				delete Decl;
			}
		}

		FTypeQualifier::FTypeQualifier()
		{
			Raw = 0;
		}

		void FTypeQualifier::Write(FASTWriter& Writer) const
		{
			if (bIsStatic)
			{
				Writer << TEXT("static ");
			}

			if (bConstant)
			{
				Writer << TEXT("const ");
			}

			if (bShared)
			{
				Writer << TEXT("groupshared ");
			}
			else if (bIn && bOut)
			{
				Writer << TEXT("inout ");
			}
			else if (bIn)
			{
				Writer << TEXT("in ");
			}
			else if (bOut)
			{
				Writer << TEXT("out ");
			}

			if (bLinear)
			{
				Writer << TEXT("linear ");
			}
			if (bCentroid)
			{
				Writer << TEXT("centroid ");
			}
			if (NoInterpolatorType)
			{
				Writer << NoInterpolatorType;
				Writer << TEXT(' ');
			}
			if (NoPerspectiveType)
			{
				Writer << NoPerspectiveType;
				Writer << TEXT(' ');
			}
			if (bSample)
			{
				Writer << TEXT("sample ");
			}

			if (bRowMajor)
			{
				Writer << TEXT("row_major ");
			}

			if (PrimitiveType)
			{
				Writer << PrimitiveType;
				Writer << TEXT(' ');
			}
		}

		FFullySpecifiedType::FFullySpecifiedType(FLinearAllocator* InAllocator, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			Specifier(nullptr)
		{
		}

		void FFullySpecifiedType::Write(FASTWriter& Writer) const
		{
			Qualifier.Write(Writer);
			Specifier->Write(Writer);
		}

		FFullySpecifiedType::~FFullySpecifiedType()
		{
			delete Specifier;
		}

		FSemanticSpecifier::FSemanticSpecifier(FLinearAllocator* InAllocator, FSemanticSpecifier::ESpecType InType, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			Arguments(InAllocator),
			Type(InType),
			Semantic(nullptr)
		{
		}

		FSemanticSpecifier::FSemanticSpecifier(FLinearAllocator* InAllocator, const TCHAR* InSemantic, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			Arguments(InAllocator),
			Type(FSemanticSpecifier::ESpecType::Semantic)
		{
			Semantic = InAllocator->Strdup(InSemantic);
		}

		FSemanticSpecifier::~FSemanticSpecifier()
		{
			for (FExpression* Expr : Arguments)
			{
				delete Expr;
			}
		}

		void FSemanticSpecifier::Write(FASTWriter& Writer) const
		{
			Writer << TEXT(" : ");
			switch (Type)
			{
			case ESpecType::Semantic:
				Writer << Semantic;
				Writer << TEXT(" ");
				break;
			case ESpecType::Register:
				Writer << TEXT("register");
				break;
			case ESpecType::PackOffset:
				Writer << TEXT("packoffset");
				break;
			default:
				Writer << *FString::Printf(TEXT("<Unknown Type value %d!>"), (int32)Type);
			}
			if (Arguments.Num() > 0)
			{
				Writer << TEXT("(");
				for (int32 Index = 0, Num = Arguments.Num(); Index < Num; ++Index)
				{
					Arguments[Index]->Write(Writer);
					if (Index + 1 < Num)
					{
						Writer << TEXT(", ");
					}
				}
				Writer << TEXT(")");
			}
		}

		FDeclaration::FDeclaration(FLinearAllocator* InAllocator, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			Identifier(nullptr),
			Semantic(nullptr),
			bIsArray(false),
			ArraySize(InAllocator),
			Initializer(nullptr)
		{
		}

		void FDeclaration::Write(FASTWriter& Writer) const
		{
			WriteAttributes(Writer);
			Writer << Identifier;

			WriteOptionArraySize(Writer, bIsArray, ArraySize);

			if (Initializer)
			{
				Writer << TEXT(" = ");
				Initializer->Write(Writer);
			}

			if (Semantic)
			{
				Semantic->Write(Writer);
			}
		}

		FDeclaration::~FDeclaration()
		{
			for (FExpression* Expr : ArraySize)
			{
				delete Expr;
			}
			delete Initializer;
		}

		FDeclaratorList::FDeclaratorList(FLinearAllocator* InAllocator, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			Type(nullptr),
			Declarations(InAllocator)
		{
		}

		void FDeclaratorList::WriteNoEOL(FASTWriter& Writer) const
		{
			WriteAttributes(Writer);
			if (bTypedef)
			{
				Writer << TEXT("typedef ");
			}

			if (Type)
			{
				Type->Write(Writer);
				Writer << TEXT(" ");
			}

			bool bFirst = true;
			for (FNode* Decl : Declarations)
			{
				if (bFirst)
				{
					bFirst = false;
				}
				else
				{
					Writer << TEXT(", ");
				}

				Decl->Write(Writer);
			}

			Writer << TEXT(";");
		}

		void FDeclaratorList::Write(FASTWriter& Writer) const
		{
			Writer.DoIndent();

			WriteNoEOL(Writer);

			Writer << TEXT("\n");
		}

		FDeclaratorList::~FDeclaratorList()
		{
			delete Type;

			for (FNode* Decl : Declarations)
			{
				delete Decl;
			}
		}

		FExpressionList::FExpressionList(FLinearAllocator* InAllocator, FExpressionList::EType InType, const FSourceInfo& InInfo) :
			FExpression(InAllocator, EOperators::ExpressionList, InInfo),
			Type(InType)
		{
		}

		void FExpressionList::Write(FASTWriter& Writer) const
		{
			switch (Type)
			{
			case EType::Braced:			Writer << TEXT("{"); break;
			case EType::Parenthesized:	Writer << TEXT("("); break;
			case EType::Templated:		Writer << TEXT("<"); break;
			default: break;
			}
			bool bFirst = true;
			for (FExpression* Expr : Expressions)
			{
				if (bFirst)
				{
					bFirst = false;
				}
				else
				{
					Writer << TEXT(", ");
				}

				Expr->Write(Writer);
			}
			switch (Type)
			{
			case EType::Braced:			Writer << TEXT("}"); break;
			case EType::Parenthesized:	Writer << TEXT(")"); break;
			case EType::Templated:		Writer << TEXT(">"); break;
			default: break;
			}
		}

		FParameterDeclarator::FParameterDeclarator(FLinearAllocator* InAllocator, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			Type(nullptr),
			Identifier(nullptr),
			Semantic(nullptr),
			bIsArray(false),
			ArraySize(InAllocator),
			DefaultValue(nullptr)
		{
		}

		void FParameterDeclarator::Write(FASTWriter& Writer) const
		{
			WriteAttributes(Writer);
			Type->Write(Writer);
			Writer << TEXT(' ') << Identifier;

			WriteOptionArraySize(Writer, bIsArray, ArraySize);

			if (Semantic)
			{
				Semantic->Write(Writer);
			}

			if (DefaultValue)
			{
				Writer << TEXT(" = ");
				DefaultValue->Write(Writer);
			}
		}

		FParameterDeclarator* FParameterDeclarator::CreateFromDeclaratorList(FDeclaratorList* List, FLinearAllocator* Allocator)
		{
			check(List);
			check(List->Declarations.Num() == 1);

			auto* Source = (FDeclaration*)List->Declarations[0];
			auto* New = new(Allocator) FParameterDeclarator(Allocator, Source->SourceInfo);
			New->Type = List->Type;
			New->Identifier = Source->Identifier;
			New->Semantic = Source->Semantic;
			New->bIsArray = Source->bIsArray;
			New->ArraySize = Source->ArraySize;
			New->DefaultValue = Source->Initializer;
			return New;
		}

		FParameterDeclarator::~FParameterDeclarator()
		{
			delete Type;

			for (FExpression* Expr : ArraySize)
			{
				delete Expr;
			}

			delete DefaultValue;
		}

		FIterationStatement::FIterationStatement(FLinearAllocator* InAllocator, const FSourceInfo& InInfo, EIterationType InType) :
			FNode(InAllocator, InInfo),
			Type(InType),
			InitStatement(nullptr),
			Condition(nullptr),
			RestExpression(nullptr),
			Body(nullptr)
		{
		}

		void FIterationStatement::Write(FASTWriter& Writer) const
		{
			Writer.DoIndent();
			WriteAttributes(Writer);
			switch (Type)
			{
			case EIterationType::For:
				Writer << TEXT("for (");
				if (InitStatement)
				{
					FDeclaratorList* DeclList = InitStatement->AsDeclaratorList();
					if (DeclList)
					{
						DeclList->WriteNoEOL(Writer);
					}
					else
					{
						InitStatement->Write(Writer);
						Writer << TEXT(";");
					}
				}
				else
				{
					Writer << TEXT(" ;");
				}
				Writer << TEXT(" ");
				if (Condition)
				{
					Condition->Write(Writer);
				}
				Writer << TEXT("; ");
				if (RestExpression)
				{
					RestExpression->Write(Writer);
				}

				Writer << TEXT(")\n");
				if (Body)
				{
					Body->Write(Writer);
				}
				else
				{
					Writer.DoIndent();
					Writer << TEXT("{\n");
					Writer.DoIndent();
					Writer << TEXT("}\n");
				}
				break;

			case EIterationType::While:
				Writer << TEXT("while (");
				Condition->Write(Writer);
				Writer << TEXT(")\n");
				Writer.DoIndent();
				Writer << TEXT("{\n");
				if (Body)
				{
					FASTWriterIncrementScope Scope(Writer);
					Body->Write(Writer);
				}
				Writer.DoIndent();
				Writer << TEXT("}\n");
				break;

			case EIterationType::DoWhile:
				Writer << TEXT("do\n");
				Writer.DoIndent();
				Writer << TEXT("{\n");
				if (Body)
				{
					FASTWriterIncrementScope Scope(Writer);
					Body->Write(Writer);
				}
				Writer.DoIndent();
				Writer << TEXT("}\n");
				Writer.DoIndent();
				Writer << TEXT("while (");
				Condition->Write(Writer);
				Writer << TEXT(");\n");
				break;

			default:
				checkf(0, TEXT("Unhandled AST iteration type %d!"), (int32)Type);
				break;
			}
		}

		FIterationStatement::~FIterationStatement()
		{
			delete InitStatement;
			delete Condition;
			delete RestExpression;
			delete Body;
		}

		FFunctionExpression::FFunctionExpression(FLinearAllocator* InAllocator, const FSourceInfo& InInfo, FExpression* InCallee) :
			FExpression(InAllocator, EOperators::FunctionCall, InInfo)
			, Callee(InCallee)
		{
		}

		void FFunctionExpression::Write(FASTWriter& Writer) const
		{
			Callee->Write(Writer);
			Writer << TEXT('(');
			bool bFirst = true;
			for (FExpression* Expr : Expressions)
			{
				if (bFirst)
				{
					bFirst = false;
				}
				else
				{
					Writer << TEXT(", ");
				}
				Expr->Write(Writer);
			}
			Writer << TEXT(")");
		}

		FSwitchStatement::FSwitchStatement(FLinearAllocator* InAllocator, const FSourceInfo& InInfo, FExpression* InCondition, FSwitchBody* InBody) :
			FNode(InAllocator, InInfo),
			Condition(InCondition),
			Body(InBody)
		{
		}

		void FSwitchStatement::Write(FASTWriter& Writer) const
		{
			Writer.DoIndent();
			Writer << TEXT("switch (");
			Condition->Write(Writer);
			Writer << TEXT(")\n");
			Body->Write(Writer);
		}

		FSwitchStatement::~FSwitchStatement()
		{
			delete Condition;
			delete Body;
		}

		FSwitchBody::FSwitchBody(FLinearAllocator* InAllocator, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			CaseList(nullptr)
		{
		}

		void FSwitchBody::Write(FASTWriter& Writer) const
		{
			Writer.DoIndent();
			Writer << TEXT("{\n");
			{
				FASTWriterIncrementScope Scope(Writer);
				CaseList->Write(Writer);
			}
			Writer.DoIndent();
			Writer << TEXT("}\n");
		}

		FSwitchBody::~FSwitchBody()
		{
			delete CaseList;
		}

		FCaseLabel::FCaseLabel(FLinearAllocator* InAllocator, const FSourceInfo& InInfo, AST::FExpression* InExpression) :
			FNode(InAllocator, InInfo),
			TestExpression(InExpression)
		{
		}

		void FCaseLabel::Write(FASTWriter& Writer) const
		{
			Writer.DoIndent();
			if (TestExpression)
			{
				Writer << TEXT("case ");
				TestExpression->Write(Writer);
			}
			else
			{
				Writer << TEXT("default");
			}

			Writer << TEXT(":\n");
		}

		FCaseLabel::~FCaseLabel()
		{
			delete TestExpression;
		}


		FCaseStatement::FCaseStatement(FLinearAllocator* InAllocator, const FSourceInfo& InInfo, FCaseLabelList* InLabels) :
			FNode(InAllocator, InInfo),
			Labels(InLabels),
			Statements(InAllocator)
		{
		}

		void FCaseStatement::Write(FASTWriter& Writer) const
		{
			Labels->Write(Writer);

			if (Statements.Num() > 1)
			{
				Writer.DoIndent();
				Writer << TEXT("{\n");
				for (auto* Statement : Statements)
				{
					FASTWriterIncrementScope Scope(Writer);
					Statement->Write(Writer);
				}
				Writer.DoIndent();
				Writer << TEXT("}\n");
			}
			else if (Statements.Num() > 0)
			{
				FASTWriterIncrementScope Scope(Writer);
				Statements[0]->Write(Writer);
			}
		}

		FCaseStatement::~FCaseStatement()
		{
			delete Labels;
			for (FNode* Statement : Statements)
			{
				delete Statement;
			}
		}

		FCaseLabelList::FCaseLabelList(FLinearAllocator* InAllocator, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			Labels(InAllocator)
		{
		}

		void FCaseLabelList::Write(FASTWriter& Writer) const
		{
			for (FCaseLabel* Label : Labels)
			{
				Label->Write(Writer);
			}
		}

		FCaseLabelList::~FCaseLabelList()
		{
			for (FCaseLabel* Label : Labels)
			{
				delete Label;
			}
		}

		FCaseStatementList::FCaseStatementList(FLinearAllocator* InAllocator, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			Cases(InAllocator)
		{
		}

		void FCaseStatementList::Write(FASTWriter& Writer) const
		{
			for (FCaseStatement* Case : Cases)
			{
				Case->Write(Writer);
			}
		}

		FCaseStatementList::~FCaseStatementList()
		{
			for (FCaseStatement* Case : Cases)
			{
				delete Case;
			}
		}

		FStructSpecifier::FStructSpecifier(FLinearAllocator* InAllocator, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			Name(nullptr),
			ParentName(nullptr),
			Members(InAllocator),
			bForwardDeclaration(false)
		{
		}

		void FStructSpecifier::Write(FASTWriter& Writer) const
		{
			Writer << TEXT("struct ");
			Writer << (Name ? Name : TEXT(""));
			if (ParentName && *ParentName)
			{
				Writer << TEXT(" : ");
				Writer << ParentName;
			}
			if (bForwardDeclaration)
			{
				Writer << TEXT(";\n");
			}
			else
			{
				Writer << TEXT('\n');
				Writer.DoIndent();
				Writer << TEXT("{\n");

				for (FNode* Member : Members)
				{
					FASTWriterIncrementScope Scope(Writer);
					Member->Write(Writer);
				}

				Writer.DoIndent();
				Writer << TEXT("}");
			}
		}

		FStructSpecifier::~FStructSpecifier()
		{
			for (FNode* Member : Members)
			{
				delete Member;
			}
		}

		FAttribute::FAttribute(FLinearAllocator* InAllocator, const FSourceInfo& InInfo, const TCHAR* InName) :
			FNode(InAllocator, InInfo),
			Name(InName),
			Arguments(InAllocator)
		{
		}

		void FAttribute::Write(FASTWriter& Writer) const
		{
			Writer << (TCHAR)'[';
			Writer << Name;

			bool bFirst = true;
			for (FAttributeArgument* Arg : Arguments)
			{
				if (bFirst)
				{
					Writer << TEXT('(');
					bFirst = false;
				}
				else
				{
					Writer << TEXT(", ");
				}

				Arg->Write(Writer);
			}

			if (!bFirst)
			{
				Writer << TEXT(")");
			}

			Writer << TEXT("]");
		}

		FAttribute::~FAttribute()
		{
			for (FAttributeArgument* Arg : Arguments)
			{
				delete Arg;
			}
		}

		FAttributeArgument::FAttributeArgument(FLinearAllocator* InAllocator, const FSourceInfo& InInfo) :
			FNode(InAllocator, InInfo),
			StringArgument(nullptr),
			ExpressionArgument(nullptr)
		{
		}

		void FAttributeArgument::Write(FASTWriter& Writer) const
		{
			if (ExpressionArgument)
			{
				ExpressionArgument->Write(Writer);
			}
			else
			{
				Writer << TEXT('"');
				Writer << StringArgument;
				Writer << TEXT('"');
			}
		}

		FAttributeArgument::~FAttributeArgument()
		{
			delete ExpressionArgument;
		}
	}
}
