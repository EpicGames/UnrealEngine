// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HlslAST.h - Abstract Syntax Tree interfaces for HLSL.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "HlslUtils.h"
#include "HlslLexer.h"

namespace CrossCompiler
{
	namespace AST
	{
		struct FCBufferDeclaration;
		struct FDeclaratorList;
		struct FDeclaration;
		struct FFunctionDefinition;
		struct FParameterDeclarator;
		struct FUnaryExpression;
		struct FAttribute;
		struct FJumpStatement;
		struct FSelectionStatement;
		struct FSwitchStatement;
		struct FIterationStatement;
		struct FCompoundStatement;
		struct FExpressionStatement;
		struct FSemanticSpecifier;

		// Wrapper for identifier strings.
		// Provides an indirection that can be used for efficient symbol renaming, etc.
		class FIdentifier
		{
		private:
			FIdentifier(const TCHAR* InString, int32 InLength)
				: StringView(InString, InLength)
			{
			}
			~FIdentifier()
			{
				// We don't expect this to ever get destroyed, as we only allocate everything from the linear allocator
				checkNoEntry();
			}

		public:

			// NOTE: All memory is owned by the liner allocator and is released when it is destroyed. No need to delete it explicitly.
			static FIdentifier* New(FLinearAllocator* Allocator, const TCHAR* InString, int32 InLength = -1)
			{
				if (InLength < 0)
				{
					InLength = FCString::Strlen(InString);
				}
				const TCHAR* DuplicatedString = Allocator->Strdup(InString, InLength);
				void* IdentifierMemory = Allocator->Alloc(sizeof(FIdentifier), alignof(FIdentifier));
				return new(IdentifierMemory) FIdentifier(DuplicatedString, InLength);
			}

			static FIdentifier* New(FLinearAllocator* Allocator, FStringView InString)
			{
				return FIdentifier::New(Allocator, InString.GetData(), InString.Len());
			}

			FStringView ToStringView() const
			{
				return StringView;
			};

			FString ToString() const
			{
				return FString(StringView);
			};

			const TCHAR* GetData() const
			{
				return StringView.GetData();
			}

			int32 Len() const
			{
				return StringView.Len();
			}

			bool Equals(FStringView Other) const
			{
				return Other.Equals(StringView, ESearchCase::CaseSensitive);
			}

			void Rename(FLinearAllocator* Allocator, FStringView InString)
			{
				int32 NewLength = InString.Len();
				const TCHAR* NewData = Allocator->Strdup(InString.GetData(), InString.Len());
				StringView = FStringView(NewData, NewLength);
			}

		private:
			FStringView StringView;
		};

		struct FASTWriter
		{
			FString& Output;
			int32 Indent = 0;
			int32 ExpressionScope = 0;

			FASTWriter(FString& FinalOutput) :
				Output(FinalOutput)
			{
			}

			// Construct from another to go back to unindented writing
			FASTWriter(FASTWriter& IndentedWriter) :
				Output(IndentedWriter.Output)
			{
			}

			void DoIndent();

			inline FASTWriter& operator << (const TCHAR* String)
			{
				if (String)
				{
					Output += String;
				}

				return *this;
			}

			inline FASTWriter& operator << (const FIdentifier* Identifier)
			{
				if (Identifier)
				{
					Output += Identifier->ToStringView();
				}

				return *this;
			}

			inline FASTWriter& operator << (const TCHAR Char)
			{
				if (Char != 0)
				{
					Output += Char;
				}

				return *this;
			}

			inline FASTWriter& operator << (uint32 N)
			{
				union
				{
					uint32 U;
					int32 I;
				} Alias;
				Alias.U = N;
				if (Alias.I < 0)
				{
					(*this) << *FString::Printf(TEXT("%uu"), N);
				}
				else
				{
					(*this) << *FString::Printf(TEXT("%u"), N);
				}
				return *this;
			}

			inline FASTWriter& operator << (float F)
			{
				if (F == 0)
				{
					(*this) << TEXT("0.0");
				}
				else
				{
					float Abs = FMath::Abs(F);
					if (Abs <= 1e-6 || Abs >= 1e6)
					{
						(*this) << *FString::Printf(TEXT("%g"), F);
					}
					else
					{
						(*this) << *FString::Printf(TEXT("%f"), F);
					}
				}
				return *this;
			}
		};

		struct FASTWriterIncrementScope
		{
			FASTWriter& Writer;

			FASTWriterIncrementScope(FASTWriter& InWriter) :
				Writer(InWriter)
			{
				++Writer.Indent;
			}

			~FASTWriterIncrementScope()
			{
				--Writer.Indent;
			}
		};

		struct FASTWriterSkipExpressionScope
		{
			FASTWriter& Writer;
			int32 OriginalScope = 0;

			FASTWriterSkipExpressionScope(FASTWriter& InWriter) :
				Writer(InWriter)
			{
				OriginalScope = Writer.ExpressionScope;
				Writer.ExpressionScope = 0;
			}

			~FASTWriterSkipExpressionScope()
			{
				Writer.ExpressionScope = OriginalScope;
			}
		};

		struct FASTWriterClearIndentScope
		{
			FASTWriter& Writer;
			int32 OriginalIndent = 0;

			FASTWriterClearIndentScope(FASTWriter& InWriter) :
				Writer(InWriter)
			{
				OriginalIndent = Writer.Indent;
				Writer.Indent = 0;
			}

			~FASTWriterClearIndentScope()
			{
				Writer.Indent = OriginalIndent;
			}
		};

		class FNode
		{
		public:
			FSourceInfo SourceInfo;

			TLinearArray<FAttribute*> Attributes;

			virtual void Write(FASTWriter& Writer) const = 0;

			// RTTI
			virtual FCBufferDeclaration* AsCBufferDeclaration() { return nullptr; }
			virtual FDeclaratorList* AsDeclaratorList() { return nullptr; }
			virtual FDeclaration* AsDeclaration() { return nullptr; }
			virtual FFunctionDefinition* AsFunctionDefinition() { return nullptr; }
			virtual FParameterDeclarator* AsParameterDeclarator() { return nullptr; }
			virtual FUnaryExpression* AsUnaryExpression() { return nullptr; }
			virtual FJumpStatement* AsJumpStatement() { return nullptr; }
			virtual FSelectionStatement* AsSelectionStatement() { return nullptr; }
			virtual FSwitchStatement* AsSwitchStatement() { return nullptr; }
			virtual FIterationStatement* AsIterationStatement() { return nullptr; }
			virtual FCompoundStatement* AsCompoundStatement() { return nullptr; }
			virtual FExpressionStatement* AsExpressionStatement() { return nullptr; }

			// Returns true if the expression can be evaluated to a constant int
			virtual bool GetConstantIntValue(int32& OutValue) const { return false; }

			/* Callers of this ralloc-based new need not call delete. It's
			* easier to just ralloc_free 'ctx' (or any of its ancestors). */
			static void* operator new(size_t Size, FLinearAllocator* Allocator)
			{
#if USE_UNREAL_ALLOCATOR
				return FMemory::Malloc(Size);
#else
				auto* Ptr = Allocator->Alloc(Size);
				return Ptr;
#endif
			}

			/* If the user *does* call delete, that's OK, we will just
			* ralloc_free in that case. */
			static void operator delete(void* Pointer)
			{
#if USE_UNREAL_ALLOCATOR
				FMemory::Free(Pointer);
#else
				// Do nothing...
#endif
			}

			virtual ~FNode() = default;

		protected:
			//FNode();
			FNode(FLinearAllocator* Allocator, const FSourceInfo& InInfo);

			void WriteAttributes(FASTWriter& Writer) const;
		};

		/**
		* Operators for AST expression nodes.
		*/
		enum class EOperators
		{
			Assign,
			Plus,        /**< Unary + operator. */
			Minus,
			Add,
			Sub,
			Mul,
			Div,
			Mod,
			LShift,
			RShift,
			Less,
			Greater,
			LEqual,
			GEqual,
			Equal,
			NEqual,
			BitAnd,
			BitXor,
			BitOr,
			BitNeg,		// ~
			LogicAnd,
			LogicXor,
			LogicOr,
			LogicNot,	// !

			MulAssign,
			DivAssign,
			ModAssign,
			AddAssign,
			SubAssign,
			LSAssign,
			RSAssign,
			AndAssign,
			XorAssign,
			OrAssign,

			Conditional,

			PreInc,
			PreDec,
			PostInc,
			PostDec,
			FieldSelection,
			ArrayIndex,

			FunctionCall,
			ExpressionList,

			Identifier,
			Literal,

			TypeCast,
			SizeOf,
		};

		inline EOperators TokenToASTOperator(EHlslToken Token)
		{
			switch (Token)
			{
			case EHlslToken::Equal:
				return AST::EOperators::Assign;

			case EHlslToken::PlusEqual:
				return AST::EOperators::AddAssign;

			case EHlslToken::MinusEqual:
				return AST::EOperators::SubAssign;

			case EHlslToken::TimesEqual:
				return AST::EOperators::MulAssign;

			case EHlslToken::DivEqual:
				return AST::EOperators::DivAssign;

			case EHlslToken::ModEqual:
				return AST::EOperators::ModAssign;

			case EHlslToken::GreaterGreaterEqual:
				return AST::EOperators::RSAssign;

			case EHlslToken::LowerLowerEqual:
				return AST::EOperators::LSAssign;

			case EHlslToken::AndEqual:
				return AST::EOperators::AndAssign;

			case EHlslToken::OrEqual:
				return AST::EOperators::OrAssign;

			case EHlslToken::XorEqual:
				return AST::EOperators::XorAssign;

			case EHlslToken::Question:
				return AST::EOperators::Conditional;

			case EHlslToken::OrOr:
				return AST::EOperators::LogicOr;

			case EHlslToken::AndAnd:
				return AST::EOperators::LogicAnd;

			case EHlslToken::Or:
				return AST::EOperators::BitOr;

			case EHlslToken::Xor:
				return AST::EOperators::BitXor;

			case EHlslToken::And:
				return AST::EOperators::BitAnd;

			case EHlslToken::EqualEqual:
				return AST::EOperators::Equal;

			case EHlslToken::NotEqual:
				return AST::EOperators::NEqual;

			case EHlslToken::Lower:
				return AST::EOperators::Less;

			case EHlslToken::Greater:
				return AST::EOperators::Greater;

			case EHlslToken::LowerEqual:
				return AST::EOperators::LEqual;

			case EHlslToken::GreaterEqual:
				return AST::EOperators::GEqual;

			case EHlslToken::LowerLower:
				return AST::EOperators::LShift;

			case EHlslToken::GreaterGreater:
				return AST::EOperators::RShift;

			case EHlslToken::Plus:
				return AST::EOperators::Add;

			case EHlslToken::Minus:
				return AST::EOperators::Sub;

			case EHlslToken::Times:
				return AST::EOperators::Mul;

			case EHlslToken::Div:
				return AST::EOperators::Div;

			case EHlslToken::Mod:
				return AST::EOperators::Mod;

			default:
				checkf(0, TEXT("Unhandled token %d"), (int32)Token);
				break;
			}

			return AST::EOperators::Plus;
		}

		struct FPragma : public FNode
		{
			FPragma(FLinearAllocator* InAllocator, const TCHAR* InPragma, const FSourceInfo& InInfo);

			virtual void Write(FASTWriter& Writer) const override;

			const TCHAR* Pragma;
		};

		struct FExpression : public FNode
		{
			FExpression(FLinearAllocator* InAllocator, EOperators InOperator, const FSourceInfo& InInfo);
			FExpression(FLinearAllocator* InAllocator, EOperators InOperator, FExpression* E0, const FSourceInfo& InInfo);
			FExpression(FLinearAllocator* InAllocator, EOperators InOperator, FExpression* E0, FExpression* E1, const FSourceInfo& InInfo);
			FExpression(FLinearAllocator* InAllocator, EOperators InOperator, FExpression* E0, FExpression* E1, FExpression* E2, const FSourceInfo& InInfo);
			~FExpression();

			EOperators Operator;

			union
			{
				ELiteralType LiteralType;
				struct FTypeSpecifier* TypeSpecifier;
			};
			const FIdentifier* Identifier;

			TLinearArray<FExpression*> Expressions;

			void WriteOperator(FASTWriter& Writer) const;
			virtual void Write(FASTWriter& Writer) const override;

			virtual bool GetConstantIntValue(int32& OutValue) const override;

			bool IsConstant() const
			{
				return Operator == EOperators::Literal;
			}
		};

		struct FUnaryExpression : public FExpression
		{
			FUnaryExpression(FLinearAllocator* InAllocator, EOperators InOperator, FExpression* Expr, const FSourceInfo& InInfo);

			virtual void Write(FASTWriter& Writer) const override;
			virtual FUnaryExpression* AsUnaryExpression() override { return this; }
		};

		struct FBinaryExpression : public FExpression
		{
			FBinaryExpression(FLinearAllocator* InAllocator, EOperators InOperator, FExpression* E0, FExpression* E1, const FSourceInfo& InInfo);

			virtual void Write(FASTWriter& Writer) const override;
			virtual bool GetConstantIntValue(int32& OutValue) const override;
		};

		struct FFunctionExpression : public FExpression
		{
			FFunctionExpression(FLinearAllocator* InAllocator, const FSourceInfo& InInfo, FExpression* InCallee);

			virtual void Write(FASTWriter& Writer) const override;

			FExpression* Callee;
		};

		struct FExpressionList : public FExpression
		{
			enum class EType
			{
				FreeForm,			// a, b, c
				Parenthesized,		// ( a, b, c )
				Braced,				// { a, b, c }
				Templated,			// < a, b, c >
			};

			FExpressionList(FLinearAllocator* InAllocator, EType InType, const FSourceInfo& InInfo);

			EType Type = EType::FreeForm;

			virtual void Write(FASTWriter& Writer) const override;
		};

		struct FCompoundStatement : public FNode
		{
			FCompoundStatement(FLinearAllocator* InAllocator, const FSourceInfo& InInfo);
			~FCompoundStatement();

			TLinearArray<FNode*> Statements;

			virtual void Write(FASTWriter& Writer) const override;
			virtual FCompoundStatement* AsCompoundStatement() override { return this; }
		};

		struct FStaticAssertStatement : public FNode
		{
			FStaticAssertStatement(FLinearAllocator* InAllocator, const FSourceInfo& InInfo, const FString& InKeyword);
			~FStaticAssertStatement();

			virtual void Write(FASTWriter& Writer) const override;

			const FString Keyword; // Either "_Static_assert" (DXC) or "static_assert" (C++11 like compilers)
			FExpression* Condition;
			FString Message;
		};

		struct FC99PragmaStatement : public FNode
		{
			FC99PragmaStatement(FLinearAllocator* InAllocator, const FSourceInfo& InInfo, const FString& InArgument);
			~FC99PragmaStatement();

			virtual void Write(FASTWriter& Writer) const override;

			FString Argument;
		};

		struct FSemanticSpecifier : public FNode
		{
			enum class ESpecType
			{
				Semantic,
				Register,
				PackOffset,
			};

			FSemanticSpecifier(FLinearAllocator* InAllocator, ESpecType InType, const FSourceInfo& InInfo);
			FSemanticSpecifier(FLinearAllocator* InAllocator, const TCHAR* InSemantic, const FSourceInfo& InInfo);
			~FSemanticSpecifier();

			virtual void Write(FASTWriter& Writer) const override;

			TLinearArray<FExpression*>	Arguments;
			ESpecType					Type;
			const TCHAR*				Semantic;
		};

		struct FDeclaration : public FNode
		{
			FDeclaration(FLinearAllocator* InAllocator, const FSourceInfo& InInfo);
			~FDeclaration();

			virtual void Write(FASTWriter& Writer) const override;
			virtual FDeclaration* AsDeclaration() override { return this; }

			FIdentifier* Identifier; // non-const to allow renaming

			FSemanticSpecifier* Semantic;

			bool bIsArray;

			TLinearArray<FExpression*> ArraySize;

			FExpression* Initializer;
		};

		struct FTypeQualifier
		{
			union
			{
				struct
				{
					uint32 bIsStatic : 1;
					uint32 bConstant : 1;
					uint32 bIn : 1;
					uint32 bOut : 1;
					uint32 bRowMajor : 1;
					uint32 bShared : 1;
					uint32 bUniform : 1;

					// Interpolation modifiers
					uint32 bLinear : 1;
					uint32 bCentroid : 1;
					uint32 bSample : 1;
				};
				uint32 Raw;
			};

			const TCHAR* PrimitiveType = nullptr;
			const TCHAR* NoInterpolatorType = nullptr;
			const TCHAR* NoPerspectiveType = nullptr;

			FTypeQualifier();

			void Write(FASTWriter& Writer) const;
		};

		struct FStructSpecifier : public FNode
		{
			FStructSpecifier(FLinearAllocator* InAllocator, const FSourceInfo& InInfo);
			~FStructSpecifier();

			virtual void Write(FASTWriter& Writer) const override;

			const TCHAR* Name;
			const TCHAR* ParentName;
			TLinearArray<FNode*> Members;
			bool bForwardDeclaration;
		};

		struct FCBufferDeclaration : public FNode
		{
			FCBufferDeclaration(FLinearAllocator* InAllocator, const FSourceInfo& InInfo);
			~FCBufferDeclaration();

			virtual void Write(FASTWriter& Writer) const override;
			virtual FCBufferDeclaration* AsCBufferDeclaration() override { return this; }

			const TCHAR* Name;
			TLinearArray<FNode*> Declarations;
		};

		struct FTypeSpecifier : public FNode
		{
			FTypeSpecifier(FLinearAllocator* InAllocator, const FSourceInfo& InInfo);
			~FTypeSpecifier();

			virtual void Write(FASTWriter& Writer) const override;

			const TCHAR* TypeName;
			const TCHAR* InnerType;

			FStructSpecifier* Structure;

			int32 TextureMSNumSamples;

			int32 PatchSize;
			bool bPrecise = false;
			bool bIsArray;
			FExpression* ArraySize;
		};

		struct FFullySpecifiedType : public FNode
		{
			FFullySpecifiedType(FLinearAllocator* InAllocator, const FSourceInfo& InInfo);
			~FFullySpecifiedType();

			virtual void Write(FASTWriter& Writer) const override;

			FTypeQualifier Qualifier;
			FTypeSpecifier* Specifier;
		};

		struct FDeclaratorList : public FNode
		{
			FDeclaratorList(FLinearAllocator* InAllocator, const FSourceInfo& InInfo);
			~FDeclaratorList();

			void WriteNoEOL(FASTWriter& Writer) const;

			virtual void Write(FASTWriter& Writer) const override;
			virtual FDeclaratorList* AsDeclaratorList() override { return this; }

			FFullySpecifiedType* Type;
			TLinearArray<FNode*> Declarations;

			bool bTypedef = false;
		};

		struct FParameterDeclarator : public FNode
		{
			FParameterDeclarator(FLinearAllocator* InAllocator, const FSourceInfo& InInfo);
			~FParameterDeclarator();

			static FParameterDeclarator* CreateFromDeclaratorList(FDeclaratorList* List, FLinearAllocator* Allocator);

			virtual void Write(FASTWriter& Writer) const override;
			virtual FParameterDeclarator* AsParameterDeclarator() override { return this; }

			FFullySpecifiedType* Type;
			FIdentifier* Identifier; // non-const to allow renaming
			FSemanticSpecifier* Semantic;
			bool bIsArray;

			TLinearArray<FExpression*> ArraySize;
			FExpression* DefaultValue;
		};

		struct FFunction : public FNode
		{
			FFunction(FLinearAllocator* InAllocator, const FSourceInfo& InInfo);
			~FFunction();

			virtual void Write(FASTWriter& Writer) const override;

			FFullySpecifiedType* ReturnType;
			const FIdentifier* ScopeIdentifier;
			const FIdentifier* Identifier;
			FSemanticSpecifier* ReturnSemantic;

			TLinearArray<FNode*> Parameters;

			bool bIsDefinition;
			bool bIsOperator;
			bool bIsStatic;

			//Signature
		};

		struct FExpressionStatement : public FNode
		{
			FExpressionStatement(FLinearAllocator* InAllocator, FExpression* InExpr, const FSourceInfo& InInfo);
			~FExpressionStatement();

			FExpression* Expression;
			virtual FExpressionStatement* AsExpressionStatement() override { return this; }

			virtual void Write(FASTWriter& Writer) const override;
		};

		struct FCaseLabel : public FNode
		{
			FCaseLabel(FLinearAllocator* InAllocator, const FSourceInfo& InInfo, AST::FExpression* InExpression);
			~FCaseLabel();

			virtual void Write(FASTWriter& Writer) const override;

			FExpression* TestExpression;
		};

		struct FCaseLabelList : public FNode
		{
			FCaseLabelList(FLinearAllocator* InAllocator, const FSourceInfo& InInfo);
			~FCaseLabelList();

			virtual void Write(FASTWriter& Writer) const override;

			TLinearArray<FCaseLabel*> Labels;
		};

		struct FCaseStatement : public FNode
		{
			FCaseStatement(FLinearAllocator* InAllocator, const FSourceInfo& InInfo, FCaseLabelList* InLabels);
			~FCaseStatement();

			virtual void Write(FASTWriter& Writer) const override;

			FCaseLabelList* Labels;
			TLinearArray<FNode*> Statements;
		};

		struct FCaseStatementList : public FNode
		{
			FCaseStatementList(FLinearAllocator* InAllocator, const FSourceInfo& InInfo);
			~FCaseStatementList();

			virtual void Write(FASTWriter& Writer) const override;

			TLinearArray<FCaseStatement*> Cases;
		};

		struct FSwitchBody : public FNode
		{
			FSwitchBody(FLinearAllocator* InAllocator, const FSourceInfo& InInfo);
			~FSwitchBody();

			virtual void Write(FASTWriter& Writer) const override;

			FCaseStatementList* CaseList;
		};

		struct FSelectionStatement : public FNode
		{
			FSelectionStatement(FLinearAllocator* InAllocator, const FSourceInfo& InInfo);
			~FSelectionStatement();

			virtual void Write(FASTWriter& Writer) const override;
			virtual FSelectionStatement* AsSelectionStatement() override { return this; }

			FExpression* Condition;

			FNode* ThenStatement;
			FNode* ElseStatement;
		};

		struct FSwitchStatement : public FNode
		{
			FSwitchStatement(FLinearAllocator* InAllocator, const FSourceInfo& InInfo, FExpression* InCondition, FSwitchBody* InBody);
			~FSwitchStatement();

			virtual void Write(FASTWriter& Writer) const override;
			virtual FSwitchStatement* AsSwitchStatement() override { return this; }

			FExpression* Condition;
			FSwitchBody* Body;
		};

		enum class EIterationType
		{
			For,
			While,
			DoWhile,
		};

		struct FIterationStatement : public FNode
		{
			FIterationStatement(FLinearAllocator* InAllocator, const FSourceInfo& InInfo, EIterationType InType);
			~FIterationStatement();

			virtual void Write(FASTWriter& Writer) const override;
			virtual FIterationStatement* AsIterationStatement() override { return this; }

			EIterationType Type;

			AST::FNode* InitStatement;
			AST::FNode* Condition;
			FExpression* RestExpression;

			AST::FNode* Body;
		};

		enum class EJumpType
		{
			Continue,
			Break,
			Return,
			//Discard,
		};

		struct FJumpStatement : public FNode
		{
			FJumpStatement(FLinearAllocator* InAllocator, EJumpType InType, const FSourceInfo& InInfo);
			~FJumpStatement();

			EJumpType Type;
			FExpression* OptionalExpression;

			virtual void Write(FASTWriter& Writer) const override;
			virtual FJumpStatement* AsJumpStatement() override { return this; }
		};

		struct FFunctionDefinition : public FNode
		{
			FFunctionDefinition(FLinearAllocator* InAllocator, const FSourceInfo& InInfo);
			~FFunctionDefinition();

			FFunction* Prototype;
			FCompoundStatement* Body;

			virtual void Write(FASTWriter& Writer) const override;
			virtual FFunctionDefinition* AsFunctionDefinition() override { return this; }
		};

		struct FAttributeArgument : public FNode
		{
			FAttributeArgument(FLinearAllocator* InAllocator, const FSourceInfo& InInfo);
			~FAttributeArgument();

			virtual void Write(FASTWriter& Writer) const override;

			const TCHAR* StringArgument;
			FExpression* ExpressionArgument;
		};

		struct FAttribute : public FNode
		{
			FAttribute(FLinearAllocator* Allocator, const FSourceInfo& InInfo, const TCHAR* InName);
			~FAttribute();

			virtual void Write(FASTWriter& Writer) const override;

			const TCHAR* Name;
			TLinearArray<FAttributeArgument*> Arguments;
		};

		static inline bool IsAssignmentOperator(EOperators Operator)
		{
			switch (Operator)
			{
			case EOperators::Assign:
			case EOperators::MulAssign:
			case EOperators::DivAssign:
			case EOperators::ModAssign:
			case EOperators::AddAssign:
			case EOperators::SubAssign:
			case EOperators::LSAssign:
			case EOperators::RSAssign:
			case EOperators::AndAssign:
			case EOperators::XorAssign:
			case EOperators::OrAssign:
				return true;

			default:
				break;
			}

			return false;
		}
	}
}
