// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HlslExpressionParser.inl - Implementation for parsing hlsl expressions.
=============================================================================*/

#pragma once
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Containers/UnrealString.h"
#include "Containers/Set.h"
#include "Developer/HlslParser/Private/HlslParser.h"
#include "Developer/HlslParser/Private/HlslAST.h"
#include "Hash/xxhash.h"

class Error;

namespace CrossCompiler
{
	EParseResult ParseResultError();

	struct FSymbolScope;
	//struct FInfo;

	EParseResult ComputeExpr(FHlslScanner& Scanner, int32 MinPrec, /*FInfo& Info,*/ FSymbolScope* SymbolScope, int32 ExpressionFlags, FLinearAllocator* Allocator, AST::FExpression** OutExpression, AST::FExpression** OutTernaryExpression);
	EParseResult ParseExpressionList(EHlslToken EndListToken, FHlslScanner& Scanner, FSymbolScope* SymbolScope, EHlslToken NewStartListToken, FLinearAllocator* Allocator, AST::FExpression* OutExpression);
	EParseResult ParseExpression(FHlslScanner& Scanner, FSymbolScope* SymbolScope, int32 ExpressionFlags, FLinearAllocator* Allocator, AST::FExpression** OutExpression);
	struct FSymbolScope
	{
		enum class EType
		{
			Unknown,
			Global,
			Namespace,
			Statement,
			Do,
			While,
			For,
			If,
			ScopedDeclaration,
			Function,
		};

		// Type tag is for debugging
		const EType ScopeType;

		FLinearAllocator* Allocator;
		FSymbolScope* Parent;
		const TCHAR* Name;

		TSet/*TLinearSet*/<FString> Symbols;
		TLinearArray<FSymbolScope> Children;

		struct FIdentifierKeyFuncs : public DefaultKeyFuncs<FStringView>
		{
			static FStringView GetSetKey(FStringView K) { return K; }
			template <typename T>
			static FStringView GetSetKey(const TPair<FStringView, T>& P) { return P.Key; }
			static bool Matches(FStringView A, FStringView B) { return A.Equals(B, ESearchCase::CaseSensitive); }
			static uint32 GetKeyHash(FStringView Key)
			{
				return FXxHash64::HashBuffer(Key.GetData(), Key.Len() * sizeof(*Key.GetData())).Hash;
			}
		};

		TMap<FStringView, CrossCompiler::AST::FIdentifier*, FDefaultSetAllocator, FIdentifierKeyFuncs> Identifiers;

		FSymbolScope(FLinearAllocator* InAllocator, FSymbolScope* InParent, EType InScopeType)
			: ScopeType(InScopeType)
			, Allocator(InAllocator)
			, Parent(InParent)
			, Name(nullptr)
			, Children(InAllocator)
		{
		}
		~FSymbolScope() = default; 

		FORCENOINLINE void Add(const FString& Type)
		{
			Symbols.Emplace(Type);
		}

		FORCENOINLINE void Add(FStringView Type)
		{
			Symbols.Emplace(Type);
		}

		FORCENOINLINE void Add(const TCHAR* Type)
		{
			Symbols.Emplace(Type);
		}

		CrossCompiler::AST::FIdentifier* FindExistingIdentifier(uint32 IdentifierHash, FStringView Identifier, bool bSearchUpwards)
		{
			if (auto ExistingEntry = Identifiers.FindByHash(IdentifierHash, Identifier))
			{
				return *ExistingEntry;
			}
			else if (bSearchUpwards && Parent)
			{
				return Parent->FindExistingIdentifier(IdentifierHash, Identifier, bSearchUpwards);
			}
			else
			{
				return nullptr;
			}
		}

		CrossCompiler::AST::FIdentifier* FindOrAddIdentifier(FStringView IdentifierName)
		{
			const uint32 IdentifierHash = FIdentifierKeyFuncs::GetKeyHash(IdentifierName);

			const bool bSearchUpwards = true;
			if (CrossCompiler::AST::FIdentifier* ExistingEntry = FindExistingIdentifier(IdentifierHash, IdentifierName, bSearchUpwards))
			{
				return ExistingEntry;
			}
			else
			{
				AST::FIdentifier* NewIdentifier = AST::FIdentifier::New(Allocator, IdentifierName);
				Identifiers.AddByHash(IdentifierHash, IdentifierName, NewIdentifier);
				return NewIdentifier;
			}
		}

		static bool FindType(const FSymbolScope* Scope, const FString& Type, bool bSearchUpwards = true)
		{
			while (Scope)
			{
				if (Scope->Symbols.Contains(Type))
				{
					return true;
				}

				if (!bSearchUpwards)
				{
					return false;
				}

				Scope = Scope->Parent;
			}

			return false;
		}

		const FSymbolScope* FindNamespace(const TCHAR* Namespace) const
		{
			for (auto& Child : Children)
			{
				if (Child.Name && FCString::Strcmp(Child.Name, Namespace) == 0)
				{
					return &Child;
				}
			}

			return nullptr;
		}

		static const FSymbolScope* FindGlobalNamespace(const TCHAR* Namespace, const FSymbolScope* Scope)
		{
			return Scope->GetGlobalScope()->FindNamespace(Namespace);
		}

		const FSymbolScope* GetGlobalScope() const
		{
			const FSymbolScope* Scope = this;
			while (Scope->Parent)
			{
				Scope = Scope->Parent;
			}

			return Scope;
		}
	};

	inline AST::FIdentifier* FindOrAddIdentifier(FLinearAllocator* Allocator, FSymbolScope* SymbolScope, FStringView IdentifierName)
	{
		if (SymbolScope)
		{
			return SymbolScope->FindOrAddIdentifier(IdentifierName);
		}
		else
		{
			return AST::FIdentifier::New(Allocator, IdentifierName);
		}
	}

	struct FCreateSymbolScope
	{
		UE_NONCOPYABLE(FCreateSymbolScope)

		FSymbolScope* Original;
		FSymbolScope** Current;

		FCreateSymbolScope(FLinearAllocator* InAllocator, FSymbolScope** InCurrent, FSymbolScope::EType ScopeType) :
			Current(InCurrent)
		{
			Original = *InCurrent;
			auto& NewScope = Original->Children.Emplace_GetRef(InAllocator, Original, ScopeType);
			*Current = &NewScope;
		}

		~FCreateSymbolScope()
		{
			*Current = Original;
		}
	};

/*
	struct FInfo
	{
		int32 Indent;
		bool bEnabled;

		FInfo(bool bInEnabled = false) : Indent(0), bEnabled(bInEnabled) {}

		void Print(const FString& Message)
		{
			if (bEnabled)
			{
				FPlatformMisc::LocalPrint(*Message);
			}
		}

		void PrintTabs()
		{
			if (bEnabled)
			{
				for (int32 Index = 0; Index < Indent; ++Index)
				{
					FPlatformMisc::LocalPrint(TEXT("\t"));
				}
			}
		}

		void PrintWithTabs(const FString& Message)
		{
			PrintTabs();
			Print(Message);
		}
	};

	struct FInfoIndentScope
	{
		FInfoIndentScope(FInfo& InInfo) : Info(InInfo)
		{
			++Info.Indent;
		}

		~FInfoIndentScope()
		{
			--Info.Indent;
		}

		FInfo& Info;
	};*/

	enum ETypeFlags
	{
		ETF_VOID					= 1 << 0,
		ETF_BUILTIN_NUMERIC			= 1 << 1,
		ETF_SAMPLER_TEXTURE_BUFFER	= 1 << 2,
		ETF_USER_TYPES				= 1 << 3,
		ETF_ERROR_IF_NOT_USER_TYPE	= 1 << 4,
		ETF_UNORM					= 1 << 5,
	};

	enum EExtraQualifiers
	{
		EEQ_PRECISE		= 1 << 0,
		EEQ_UNORM		= 1 << 1,
		EEQ_SNORM		= 1 << 2,
	};

	enum EExpressionFlags
	{
		EEF_ALLOW_ASSIGNMENT	= 1 << 0,
	};

	EParseResult ParseGeneralTypeToken(const FHlslToken* Token, int32 TypeFlags, int32 ExtraQualifierFlags, FLinearAllocator* Allocator, AST::FTypeSpecifier** OutSpecifier)
	{
		if (!Token)
		{
			return ParseResultError();
		}

		bool bMatched = false;
		const TCHAR* InnerType = nullptr;
		switch (Token->Token)
		{
		case EHlslToken::Void:
			if (TypeFlags & ETF_VOID)
			{
				bMatched = true;
			}
			break;

		case EHlslToken::Bool:
		case EHlslToken::Bool1:
		case EHlslToken::Bool2:
		case EHlslToken::Bool3:
		case EHlslToken::Bool4:
		case EHlslToken::Bool1x1:
		case EHlslToken::Bool1x2:
		case EHlslToken::Bool1x3:
		case EHlslToken::Bool1x4:
		case EHlslToken::Bool2x1:
		case EHlslToken::Bool2x2:
		case EHlslToken::Bool2x3:
		case EHlslToken::Bool2x4:
		case EHlslToken::Bool3x1:
		case EHlslToken::Bool3x2:
		case EHlslToken::Bool3x3:
		case EHlslToken::Bool3x4:
		case EHlslToken::Bool4x1:
		case EHlslToken::Bool4x2:
		case EHlslToken::Bool4x3:
		case EHlslToken::Bool4x4:

		case EHlslToken::Int:
		case EHlslToken::Int1:
		case EHlslToken::Int2:
		case EHlslToken::Int3:
		case EHlslToken::Int4:
		case EHlslToken::Int1x1:
		case EHlslToken::Int1x2:
		case EHlslToken::Int1x3:
		case EHlslToken::Int1x4:
		case EHlslToken::Int2x1:
		case EHlslToken::Int2x2:
		case EHlslToken::Int2x3:
		case EHlslToken::Int2x4:
		case EHlslToken::Int3x1:
		case EHlslToken::Int3x2:
		case EHlslToken::Int3x3:
		case EHlslToken::Int3x4:
		case EHlslToken::Int4x1:
		case EHlslToken::Int4x2:
		case EHlslToken::Int4x3:
		case EHlslToken::Int4x4:

		case EHlslToken::Uint:
		case EHlslToken::Uint1:
		case EHlslToken::Uint2:
		case EHlslToken::Uint3:
		case EHlslToken::Uint4:
		case EHlslToken::Uint1x1:
		case EHlslToken::Uint1x2:
		case EHlslToken::Uint1x3:
		case EHlslToken::Uint1x4:
		case EHlslToken::Uint2x1:
		case EHlslToken::Uint2x2:
		case EHlslToken::Uint2x3:
		case EHlslToken::Uint2x4:
		case EHlslToken::Uint3x1:
		case EHlslToken::Uint3x2:
		case EHlslToken::Uint3x3:
		case EHlslToken::Uint3x4:
		case EHlslToken::Uint4x1:
		case EHlslToken::Uint4x2:
		case EHlslToken::Uint4x3:
		case EHlslToken::Uint4x4:

		case EHlslToken::Uint64_t:
		case EHlslToken::Uint64_t1:
		case EHlslToken::Uint64_t2:
		case EHlslToken::Uint64_t3:
		case EHlslToken::Uint64_t4:
		case EHlslToken::Uint64_t1x1:
		case EHlslToken::Uint64_t1x2:
		case EHlslToken::Uint64_t1x3:
		case EHlslToken::Uint64_t1x4:
		case EHlslToken::Uint64_t2x1:
		case EHlslToken::Uint64_t2x2:
		case EHlslToken::Uint64_t2x3:
		case EHlslToken::Uint64_t2x4:
		case EHlslToken::Uint64_t3x1:
		case EHlslToken::Uint64_t3x2:
		case EHlslToken::Uint64_t3x3:
		case EHlslToken::Uint64_t3x4:
		case EHlslToken::Uint64_t4x1:
		case EHlslToken::Uint64_t4x2:
		case EHlslToken::Uint64_t4x3:
		case EHlslToken::Uint64_t4x4:

		case EHlslToken::Half:
		case EHlslToken::Half1:
		case EHlslToken::Half2:
		case EHlslToken::Half3:
		case EHlslToken::Half4:
		case EHlslToken::Half1x1:
		case EHlslToken::Half1x2:
		case EHlslToken::Half1x3:
		case EHlslToken::Half1x4:
		case EHlslToken::Half2x1:
		case EHlslToken::Half2x2:
		case EHlslToken::Half2x3:
		case EHlslToken::Half2x4:
		case EHlslToken::Half3x1:
		case EHlslToken::Half3x2:
		case EHlslToken::Half3x3:
		case EHlslToken::Half3x4:
		case EHlslToken::Half4x1:
		case EHlslToken::Half4x2:
		case EHlslToken::Half4x3:
		case EHlslToken::Half4x4:

		case EHlslToken::Min16Float:
		case EHlslToken::Min16Float1:
		case EHlslToken::Min16Float2:
		case EHlslToken::Min16Float3:
		case EHlslToken::Min16Float4:
		case EHlslToken::Min16Float1x1:
		case EHlslToken::Min16Float1x2:
		case EHlslToken::Min16Float1x3:
		case EHlslToken::Min16Float1x4:
		case EHlslToken::Min16Float2x1:
		case EHlslToken::Min16Float2x2:
		case EHlslToken::Min16Float2x3:
		case EHlslToken::Min16Float2x4:
		case EHlslToken::Min16Float3x1:
		case EHlslToken::Min16Float3x2:
		case EHlslToken::Min16Float3x3:
		case EHlslToken::Min16Float3x4:
		case EHlslToken::Min16Float4x1:
		case EHlslToken::Min16Float4x2:
		case EHlslToken::Min16Float4x3:
		case EHlslToken::Min16Float4x4:

		case EHlslToken::Float:
		case EHlslToken::Float1:
		case EHlslToken::Float2:
		case EHlslToken::Float3:
		case EHlslToken::Float4:
		case EHlslToken::Float1x1:
		case EHlslToken::Float1x2:
		case EHlslToken::Float1x3:
		case EHlslToken::Float1x4:
		case EHlslToken::Float2x1:
		case EHlslToken::Float2x2:
		case EHlslToken::Float2x3:
		case EHlslToken::Float2x4:
		case EHlslToken::Float3x1:
		case EHlslToken::Float3x2:
		case EHlslToken::Float3x3:
		case EHlslToken::Float3x4:
		case EHlslToken::Float4x1:
		case EHlslToken::Float4x2:
		case EHlslToken::Float4x3:
		case EHlslToken::Float4x4:
			if (TypeFlags & ETF_BUILTIN_NUMERIC)
			{
				bMatched = true;
			}
			break;

		case EHlslToken::Texture:
		case EHlslToken::Texture1D:
		case EHlslToken::Texture1DArray:
		case EHlslToken::Texture2D:
		case EHlslToken::Texture2DArray:
		case EHlslToken::Texture2DMS:
		case EHlslToken::Texture2DMSArray:
		case EHlslToken::Texture3D:
		case EHlslToken::TextureCube:
		case EHlslToken::TextureCubeArray:

		case EHlslToken::Buffer:
		case EHlslToken::AppendStructuredBuffer:
		case EHlslToken::ConsumeStructuredBuffer:
		case EHlslToken::RWBuffer:
		case EHlslToken::RWStructuredBuffer:
		case EHlslToken::RWTexture1D:
		case EHlslToken::RWTexture1DArray:
		case EHlslToken::RWTexture2D:
		case EHlslToken::RWTexture2DArray:
		case EHlslToken::RasterizerOrderedTexture2D:
		case EHlslToken::RWTexture3D:
		case EHlslToken::ConstantBuffer:
		case EHlslToken::StructuredBuffer:
			if (TypeFlags & ETF_SAMPLER_TEXTURE_BUFFER)
			{
				bMatched = true;
				InnerType = TEXT("float4");
			}
			break;

		case EHlslToken::RaytracingAccelerationStructure:
			if (TypeFlags & ETF_SAMPLER_TEXTURE_BUFFER)
			{
				bMatched = true;
			}
			break;

		case EHlslToken::Sampler:
		case EHlslToken::Sampler1D:
		case EHlslToken::Sampler2D:
		case EHlslToken::Sampler3D:
		case EHlslToken::SamplerCube:
		case EHlslToken::SamplerState:
		case EHlslToken::SamplerComparisonState:
		case EHlslToken::ByteAddressBuffer:
		case EHlslToken::RWByteAddressBuffer:
			if (TypeFlags & ETF_SAMPLER_TEXTURE_BUFFER)
			{
				bMatched = true;
			}
			break;
		}

		if (bMatched)
		{
			//#todo-rco: Don't re-allocate types
			TStringBuilder<64> TypeName;
			TypeName += (ExtraQualifierFlags & EEQ_SNORM) == EEQ_SNORM
				? TEXTVIEW("snorm ")
				: ((ExtraQualifierFlags & EEQ_UNORM) == EEQ_UNORM
					? TEXTVIEW("unorm ")
					: TEXTVIEW(""));
			TypeName += Token->String;

			auto* Type = new(Allocator) AST::FTypeSpecifier(Allocator, Token->SourceInfo);
			Type->TypeName = Allocator->Strdup(TypeName.ToView());
			Type->InnerType = InnerType;
			Type->bPrecise = (ExtraQualifierFlags & EEQ_PRECISE) == EEQ_PRECISE;
			*OutSpecifier = Type;
			return EParseResult::Matched;
		}

		return EParseResult::NotMatched;
	}

	EParseResult ParseGeneralTypeFromToken(const FHlslToken* Token, int32 TypeFlags, int32 ExtraQualifierFlags, FSymbolScope* SymbolScope, FLinearAllocator* Allocator, AST::FTypeSpecifier** OutSpecifier)
	{
		if (Token)
		{
			if (ParseGeneralTypeToken(Token, TypeFlags, ExtraQualifierFlags, Allocator, OutSpecifier) == EParseResult::Matched)
			{
				return EParseResult::Matched;
			}

			if (TypeFlags & ETF_USER_TYPES)
			{
				check(SymbolScope);
				if (Token->Token == EHlslToken::Identifier)
				{
					if (FSymbolScope::FindType(SymbolScope, Token->String))
					{
						auto* Type = new(Allocator) AST::FTypeSpecifier(Allocator, Token->SourceInfo);
						Type->TypeName = Allocator->Strdup(Token->String);
						*OutSpecifier = Type;
						return EParseResult::Matched;
					}
					else if (TypeFlags & ETF_ERROR_IF_NOT_USER_TYPE)
					{
						return ParseResultError();
					}
				}
			}

			return EParseResult::NotMatched;
		}

		return ParseResultError();
	}

	EParseResult ParseGeneralType(FHlslScanner& Scanner, int32 TypeFlags, FSymbolScope* SymbolScope, FLinearAllocator* Allocator, AST::FTypeSpecifier** OutSpecifier)
	{
		auto* Token = Scanner.PeekToken();

		// Handle types with namespaces
		{
			auto* Token1 = Scanner.PeekToken(1);
			auto* Token2 = Scanner.PeekToken(2);
			TStringBuilder<256> TypeString;
			if (Token && Token->Token == EHlslToken::Identifier && Token1 && Token1->Token == EHlslToken::ColonColon && Token2 && Token2->Token == EHlslToken::Identifier && SymbolScope)
			{
				auto* Namespace = SymbolScope->GetGlobalScope();
				do
				{
					Token = Scanner.PeekToken();
					check(Scanner.MatchToken(EHlslToken::Identifier));
					auto* OuterNamespace = Token;
					check(Scanner.MatchToken(EHlslToken::ColonColon));
					auto* InnerOrType = Scanner.PeekToken();
					if (!InnerOrType)
					{
						Scanner.SourceError(*FString::Printf(TEXT("Expecting identifier for type '%s'!"), InnerOrType ? *InnerOrType->String : TEXT("null")));
						return ParseResultError();
					}

					Namespace = Namespace->FindNamespace(*OuterNamespace->String);
					if (!Namespace)
					{
						Scanner.SourceError(*FString::Printf(TEXT("Unknown namespace '%s'!"), *TypeString));
						return ParseResultError();
					}
					TypeString += OuterNamespace->String;
					TypeString += TEXT("::");
					auto* Token3 = Scanner.PeekToken(1);
					if (Token3 && Token3->Token == EHlslToken::ColonColon)
					{
						continue;
					}
					else
					{
						if (InnerOrType && FChar::IsAlpha(InnerOrType->String[0]))
						{
							Scanner.Advance();
							if (FSymbolScope::FindType(Namespace, InnerOrType->String, false))
							{
								auto* Type = new(Allocator) AST::FTypeSpecifier(Allocator, Token->SourceInfo);
								Type->TypeName = Allocator->Strdup(TypeString.ToView());
								*OutSpecifier = Type;
								return EParseResult::Matched;
							}
							else
							{
								Scanner.SourceError(*FString::Printf(TEXT("Unknown type '%s'!"), *TypeString));
								return ParseResultError();
							}
						}
						break;
					}
				}
				while (Token);
			}
		}

		int32 ExtraQualifiers = 0;
		bool bPrecise = false;
		if (Scanner.MatchToken(EHlslToken::Precise))
		{
			Token = Scanner.GetCurrentToken();
			ExtraQualifiers |= EEQ_PRECISE;
		}

		if ((TypeFlags & ETF_UNORM) == ETF_UNORM && Token && Token->String.Len() == 5)
		{
			if (!FCString::Strcmp(*Token->String, TEXT("unorm")))
			{
				ExtraQualifiers |= EEQ_UNORM;
				Scanner.Advance();
				Token = Scanner.GetCurrentToken();
			}
			else if (!FCString::Strcmp(*Token->String, TEXT("snorm")))
			{
				ExtraQualifiers |= EEQ_SNORM;
				Scanner.Advance();
				Token = Scanner.GetCurrentToken();
			}
		}

		auto Result = ParseGeneralTypeFromToken(Token, TypeFlags, ExtraQualifiers, SymbolScope, Allocator, OutSpecifier);
		if (Result == EParseResult::Matched)
		{
			Scanner.Advance();
			return EParseResult::Matched;
		}

		if (Result == EParseResult::Error && (TypeFlags & ETF_ERROR_IF_NOT_USER_TYPE))
		{
			Scanner.SourceError(*FString::Printf(TEXT("Unknown type '%s'!"), Token ? *Token->String : TEXT("<null>")));
			return Result;
		}

		return EParseResult::NotMatched;
	}

	EParseResult ParseFullType(FHlslScanner& Scanner, int32 TypeFlags, int32 TemplateTypeFlags, FSymbolScope* SymbolScope, FLinearAllocator* Allocator, AST::FFullySpecifiedType** OutFullySpecifiedType)
	{
		AST::FTypeSpecifier* TypeSpecifier = nullptr;
		EParseResult Result = ParseGeneralType(Scanner, TypeFlags, SymbolScope, Allocator, &TypeSpecifier);
		if (Result != EParseResult::Matched)
		{
			return Result;
		}

		// OutFullySpecifiedType could already be initialized
		AST::FFullySpecifiedType* FullType = *OutFullySpecifiedType ? *OutFullySpecifiedType : new(Allocator) AST::FFullySpecifiedType(Allocator, TypeSpecifier->SourceInfo);
		FullType->Specifier = TypeSpecifier;

		if (Scanner.MatchToken(EHlslToken::Lower))
		{
			AST::FTypeSpecifier* ElementTypeSpecifier = nullptr;
			EParseResult InnerResult = ParseGeneralType(Scanner, TemplateTypeFlags, SymbolScope, Allocator, &ElementTypeSpecifier);
			if (InnerResult != EParseResult::Matched)
			{
				Scanner.SourceError(TEXT("Expected type!"));
				return ParseResultError();
			}

			FullType->Specifier->InnerType = ElementTypeSpecifier->TypeName;

			if (Scanner.MatchToken(EHlslToken::Comma))
			{
				auto* Integer = Scanner.GetCurrentToken();
				if (!Scanner.MatchIntegerLiteral())
				{
					Scanner.SourceError(TEXT("Expected constant!"));
					return ParseResultError();
				}
				FullType->Specifier->TextureMSNumSamples = FCString::Atoi(*Integer->String);
			}

			if (!Scanner.MatchToken(EHlslToken::Greater))
			{
				Scanner.SourceError(TEXT("Expected '>'!"));
				return ParseResultError();
			}
		}

		*OutFullySpecifiedType = FullType;

		return EParseResult::Matched;
	}


	// Unary!(Unary-(Unary+())) would have ! as Top, and + as Inner
	EParseResult MatchUnaryOperator(FHlslScanner& Scanner, /*FInfo& Info,*/ FSymbolScope* SymbolScope, FLinearAllocator* Allocator, AST::FExpression** OuterExpression, AST::FExpression** InnerExpression)
	{
		bool bFoundAny = false;
		bool bTryAgain = true;
		AST::FExpression*& PrevExpression = *InnerExpression;
		while (Scanner.HasMoreTokens() && bTryAgain)
		{
			auto* Token = Scanner.GetCurrentToken();
			AST::EOperators Operator = AST::EOperators::Plus;

			switch (Token->Token)
			{
			case EHlslToken::PlusPlus:
				bFoundAny = true;
				Scanner.Advance();
				Operator = AST::EOperators::PreInc;
				break;

			case EHlslToken::MinusMinus:
				bFoundAny = true;
				Scanner.Advance();
				Operator = AST::EOperators::PreDec;
				break;

			case EHlslToken::Plus:
				Scanner.Advance();
				bFoundAny = true;
				Operator = AST::EOperators::Plus;
				break;

			case EHlslToken::Minus:
				Scanner.Advance();
				bFoundAny = true;
				Operator = AST::EOperators::Minus;
				break;

			case EHlslToken::Not:
				Scanner.Advance();
				bFoundAny = true;
				Operator = AST::EOperators::LogicNot;
				break;

			case EHlslToken::Neg:
				Scanner.Advance();
				bFoundAny = true;
				Operator = AST::EOperators::BitNeg;
				break;

			case EHlslToken::LeftParenthesis:
			// Only cast expressions are Unary
			{
				const auto* Peek1 = Scanner.PeekToken(1);
				const auto* Peek2 = Scanner.PeekToken(2);

				bool bFoundConst = false;
				int32 PeekN = 0;
				auto HandleUnaryToken = [&](EHlslToken TokenType)
				{
					if (Peek1->Token == TokenType)
					{
						++PeekN;
						Peek1 = Scanner.PeekToken(1 + PeekN);
						Peek2 = Scanner.PeekToken(2 + PeekN);
						return true;
					}
					return false;
				};

				//#todo-rco: Workaround for weird const cast in RHS codegen from HLSLTranslator:
				//		const uint exp2 = ((((const int) ((uRes32>>23)&0xff))-127+15) << 10);
				if (HandleUnaryToken(EHlslToken::Const))
				{
					HandleUnaryToken(EHlslToken::Precise);
				}
				else if (HandleUnaryToken(EHlslToken::Precise))
				{
					HandleUnaryToken(EHlslToken::Const);
				}

				AST::FTypeSpecifier* TypeSpecifier = nullptr;
				//#todo-rco: Is precise allowed on casts?
				if (Peek1 && ParseGeneralTypeFromToken(Peek1, ETF_BUILTIN_NUMERIC | ETF_USER_TYPES, 0, SymbolScope, Allocator, &TypeSpecifier) == EParseResult::Matched && Peek2 && Peek2->Token == EHlslToken::RightParenthesis)
				{
					for (; PeekN > 0; --PeekN) //-V::654,621
					{
						Scanner.Advance();
					}

					// Cast
					Scanner.Advance();
					Scanner.Advance();
					Scanner.Advance();
					bFoundAny = true;

					auto* Expression = new(Allocator) AST::FUnaryExpression(Allocator, AST::EOperators::TypeCast, nullptr, Token->SourceInfo);
					if (PrevExpression)
					{
						PrevExpression->Expressions[0] = Expression;
					}

					Expression->TypeSpecifier = TypeSpecifier;

					if (!*OuterExpression)
					{
						*OuterExpression = Expression;
					}

					PrevExpression = Expression;
					continue;
				}
				else
				{
					// Non-unary
					return bFoundAny ? EParseResult::Matched : EParseResult::NotMatched;
				}
			}
				break;

			default:
				return bFoundAny ? EParseResult::Matched : EParseResult::NotMatched;
			}

			auto* Expression = new(Allocator) AST::FUnaryExpression(Allocator, Operator, nullptr, Token->SourceInfo);
			if (PrevExpression)
			{
				PrevExpression->Expressions[0] = Expression;
			}

			if (!*OuterExpression)
			{
				*OuterExpression = Expression;
			}

			PrevExpression = Expression;
		}

		// Ran out of tokens!
		return ParseResultError();
	}

	EParseResult MatchSuffixOperator(FHlslScanner& Scanner, /*FInfo& Info,*/ FSymbolScope* SymbolScope, int32 ExpressionFlags, FLinearAllocator* Allocator, AST::FExpression** InOutExpression, AST::FExpression** OutTernaryExpression)
	{
		bool bFoundAny = false;
		bool bTryAgain = true;
		AST::FExpression*& PrevExpression = *InOutExpression;
		while (Scanner.HasMoreTokens() && bTryAgain)
		{
			auto* Token = Scanner.GetCurrentToken();
			AST::EOperators Operator = AST::EOperators::Plus;

			switch (Token->Token)
			{
			case EHlslToken::LeftSquareBracket:
			{
				Scanner.Advance();
				AST::FExpression* ArrayIndex = nullptr;
				auto Result = ComputeExpr(Scanner, 1, /*Info,*/ SymbolScope, ExpressionFlags, Allocator, &ArrayIndex, nullptr);
				if (Result != EParseResult::Matched)
				{
					Scanner.SourceError(TEXT("Expected expression!"));
					return ParseResultError();
				}

				if (!Scanner.MatchToken(EHlslToken::RightSquareBracket))
				{
					Scanner.SourceError(TEXT("Expected ']'!"));
					return ParseResultError();
				}

				auto* ArrayIndexExpression = new(Allocator) AST::FBinaryExpression(Allocator, AST::EOperators::ArrayIndex, PrevExpression, ArrayIndex, Token->SourceInfo);
				PrevExpression = ArrayIndexExpression;
				bFoundAny = true;
			}
				break;
			case EHlslToken::Dot:
			{
				Scanner.Advance();
				const auto* Identifier = Scanner.GetCurrentToken();
				if (!Scanner.MatchToken(EHlslToken::Identifier))
				{
					Scanner.SourceError(TEXT("Expected identifier for member or swizzle!"));
					return ParseResultError();
				}
				auto* FieldExpression = new(Allocator) AST::FUnaryExpression(Allocator, AST::EOperators::FieldSelection, PrevExpression, Token->SourceInfo);
				FieldExpression->Identifier = AST::FIdentifier::New(Allocator, Identifier->String);
				PrevExpression = FieldExpression;
				bFoundAny = true;
			}
				break;
			case EHlslToken::LeftParenthesis:
			{
				Scanner.Advance();

				if (PrevExpression->Identifier && PrevExpression->Identifier->ToString() == TEXT("sizeof"))
				{
					const FHlslToken* Token1 = Scanner.PeekToken();

					AST::FTypeSpecifier* TypeSpecifier = nullptr;
					auto* SizeOfExpression = new(Allocator) AST::FUnaryExpression(Allocator, AST::EOperators::SizeOf, PrevExpression, Token1->SourceInfo);

					if (ParseGeneralTypeFromToken(Token1, ETF_BUILTIN_NUMERIC, 0, SymbolScope, Allocator, &TypeSpecifier) == EParseResult::Matched)
					{
						AST::FTypeSpecifier* ElementTypeSpecifier = nullptr;
						EParseResult InnerResult = ParseGeneralType(Scanner, ETF_BUILTIN_NUMERIC, SymbolScope, Allocator, &ElementTypeSpecifier);
						if (InnerResult != EParseResult::Matched)
						{
							Scanner.SourceError(TEXT("Expected numeric type!"));
							return ParseResultError();
						}

						SizeOfExpression->Identifier = AST::FIdentifier::New(Allocator, ElementTypeSpecifier->TypeName);
					}
					else
					{
						AST::FExpression* ConditionExpression = nullptr;
						auto Result = ParseExpression(Scanner, SymbolScope, 0, Allocator, &ConditionExpression);
						if (Result == EParseResult::Error)
						{
							Scanner.SourceError(TEXT("Expected expression or declaration!\n"));
							return ParseResultError();
						}

						SizeOfExpression->Identifier = AST::FIdentifier::New(Allocator, ConditionExpression->Identifier->ToString());
					}

					if (!Scanner.MatchToken(EHlslToken::RightParenthesis))
					{
						Scanner.SourceError(TEXT("Expected )"));
						return ParseResultError();
					}

					PrevExpression = SizeOfExpression;
					bFoundAny = true;
				}
				else
				{
					// Function Call
					auto* FunctionCall = new(Allocator) AST::FFunctionExpression(Allocator, Token->SourceInfo, PrevExpression);
					auto Result = ParseExpressionList(EHlslToken::RightParenthesis, Scanner, SymbolScope, EHlslToken::Invalid, Allocator, FunctionCall);
					if (Result != EParseResult::Matched)
					{
						Scanner.SourceError(TEXT("Expected ')'!"));
						return ParseResultError();
					}

					PrevExpression = FunctionCall;

					bFoundAny = true;
				}


			}
				break;
			case EHlslToken::PlusPlus:
			{
				Scanner.Advance();
				auto* IncExpression = new(Allocator) AST::FUnaryExpression(Allocator, AST::EOperators::PostInc, PrevExpression, Token->SourceInfo);
				PrevExpression = IncExpression;
				bFoundAny = true;
			}
				break;
			case EHlslToken::MinusMinus:
			{
				Scanner.Advance();
				auto* DecExpression = new(Allocator) AST::FUnaryExpression(Allocator, AST::EOperators::PostDec, PrevExpression, Token->SourceInfo);
				PrevExpression = DecExpression;
				bFoundAny = true;
			}
				break;
			case EHlslToken::Question:
			{
				Scanner.Advance();
				AST::FExpression* Left = nullptr;
				if (ComputeExpr(Scanner, 0, /*Info,*/ SymbolScope, (ExpressionFlags | EEF_ALLOW_ASSIGNMENT), Allocator, &Left, nullptr) != EParseResult::Matched)
				{
					Scanner.SourceError(TEXT("Expected expression!"));
					return ParseResultError();
				}
				if (!Scanner.MatchToken(EHlslToken::Colon))
				{
					Scanner.SourceError(TEXT("Expected ':'!"));
					return ParseResultError();
				}
				AST::FExpression* Right = nullptr;
				if (ComputeExpr(Scanner, 0, /*Info,*/ SymbolScope, (ExpressionFlags | EEF_ALLOW_ASSIGNMENT), Allocator, &Right, nullptr) != EParseResult::Matched)
				{
					Scanner.SourceError(TEXT("Expected expression!"));
					return ParseResultError();
				}

				auto* Ternary = new(Allocator) AST::FExpression(Allocator, AST::EOperators::Conditional, nullptr, Left, Right, Token->SourceInfo);
				*OutTernaryExpression = Ternary;
				bFoundAny = true;
				bTryAgain = false;
			}
				break;
			case EHlslToken::Lower:
			{
				if (SymbolScope && SymbolScope->ScopeType == FSymbolScope::EType::Statement)
				{
					const FHlslToken* Token1 = Scanner.PeekToken(1);
					
					AST::FTypeSpecifier* TypeSpecifier = nullptr;
					if (ParseGeneralTypeFromToken(Token1, ETF_BUILTIN_NUMERIC | ETF_USER_TYPES, 0, SymbolScope, Allocator, &TypeSpecifier) == EParseResult::Matched )
					{
						Scanner.Advance();
						
						AST::FExpressionList* ExpressionList = new(Allocator) AST::FExpressionList(Allocator, AST::FExpressionList::EType::Templated, Scanner.PeekToken()->SourceInfo);

						while (Scanner.HasMoreTokens())
						{
							const auto* CurrentToken = Scanner.PeekToken();
							if (CurrentToken->Token == EHlslToken::Greater)
							{
								Scanner.Advance();
								bFoundAny = true;
								break;
							}

							if (CurrentToken->Token == EHlslToken::Comma)
							{
								Scanner.Advance();
							}
							else
							{
								AST::FTypeSpecifier* ElementTypeSpecifier = nullptr;
								EParseResult InnerResult = ParseGeneralType(Scanner, ETF_BUILTIN_NUMERIC | ETF_USER_TYPES, SymbolScope, Allocator, &ElementTypeSpecifier);
								if (InnerResult != EParseResult::Matched)
								{
									Scanner.SourceError(TEXT("Expected numeric or user type!"));
									return ParseResultError();
								}

								AST::FExpression* CurrentExpression = new(Allocator) AST::FExpression(Allocator, AST::EOperators::Identifier, CurrentToken->SourceInfo);
								ExpressionList->Expressions.Add(CurrentExpression);
							}
						}

						if (!bFoundAny)
						{
							Scanner.SourceError(TEXT("Expected >"));
							return ParseResultError();
						}

						PrevExpression = ExpressionList;
					}
					else
					{
						bTryAgain = false;
					}
				}
				else
				{
					bTryAgain = false;
				}

			}
				break;
			default:
				bTryAgain = false;
				break;
			}
		}

		*InOutExpression = PrevExpression;
		return bFoundAny ? EParseResult::Matched : EParseResult::NotMatched;
	}

	EParseResult ComputeAtom(FHlslScanner& Scanner, /*FInfo& Info,*/ FSymbolScope* SymbolScope, int32 ExpressionFlags, FLinearAllocator* Allocator, AST::FExpression** OutExpression, AST::FExpression** OutTernaryExpression)
	{
		AST::FExpression* InnerUnaryExpression = nullptr;
		auto UnaryResult = MatchUnaryOperator(Scanner, /*Info,*/ SymbolScope, Allocator, OutExpression, &InnerUnaryExpression);
		auto* Token = Scanner.GetCurrentToken();
		if (!Token || UnaryResult == EParseResult::Error)
		{
			return ParseResultError();
		}

		auto ParseScopedIdentifier = [SymbolScope](FHlslScanner& Scanner, FLinearAllocator* Allocator, FString& Name, const FHlslToken* Token, const FHlslToken* Token1, const FHlslToken* Token2)
		{
			while (Token1 && Token1->Token == EHlslToken::ColonColon && Token2 && Token2->Token == EHlslToken::Identifier)
			{
				Name += Token1->String;
				Name += Token2->String;
				Scanner.Advance();
				Scanner.Advance();
				Token1 = Scanner.PeekToken();
				Token2 = Scanner.PeekToken(1);
			}

			AST::FExpression* AtomExpression = new(Allocator) AST::FExpression(Allocator, AST::EOperators::Identifier, Token->SourceInfo);
			AtomExpression->Identifier = FindOrAddIdentifier(Allocator, SymbolScope, *Name);
			return AtomExpression;
		};

		AST::FExpression* AtomExpression = nullptr;
		switch (Token->Token)
		{
		case EHlslToken::Literal:
			Scanner.Advance();
			AtomExpression = new(Allocator) AST::FExpression(Allocator, AST::EOperators::Literal, Token->SourceInfo);
			AtomExpression->Identifier = AST::FIdentifier::New(Allocator, Token->String);
			AtomExpression->LiteralType = Token->LiteralType;
			break;

		case EHlslToken::ColonColon:
		{
			const FHlslToken* Token1 = Scanner.PeekToken(1);
			if (Token1 && Token1->Token == EHlslToken::Identifier)
			{
				FString Name;
				AtomExpression = ParseScopedIdentifier(Scanner, Allocator, Name, Token, Token, Token1);
			}
			else
			{
				Scanner.SourceError(TEXT("Expected identifier after ::"));
				return ParseResultError();
			}
		}
			break;

		case EHlslToken::Identifier:
		{
			auto* Token1 = Scanner.PeekToken(1);
			auto* Token2 = Scanner.PeekToken(2);
			if (Token1 && Token1->Token == EHlslToken::ColonColon && Token2 && Token2->Token == EHlslToken::Identifier)
			{
				FString Name = Token->String;
				Scanner.Advance();
				AtomExpression = ParseScopedIdentifier(Scanner, Allocator, Name, Token, Token1, Token2);
			}
			else
			{
				Scanner.Advance();
				AtomExpression = new(Allocator) AST::FExpression(Allocator, AST::EOperators::Identifier, Token->SourceInfo);
				AtomExpression->Identifier = FindOrAddIdentifier(Allocator, SymbolScope, Token->String);
			}
		}
			break;

		case EHlslToken::LeftParenthesis:
		{
			Scanner.Advance();

			// Parenthesis expression
			if (ComputeExpr(Scanner, 1, /*Info,*/ SymbolScope, ExpressionFlags, Allocator, &AtomExpression, nullptr) != EParseResult::Matched)
			{
				Scanner.SourceError(TEXT("Expected expression!"));
				return ParseResultError();
			}

			if (Scanner.MatchToken(EHlslToken::Comma))
			{
				AST::FExpression* FirstSequenceEntry = AtomExpression;
				AtomExpression = new(Allocator) AST::FExpressionList(Allocator, AST::FExpressionList::EType::Parenthesized, Token->SourceInfo);
				AtomExpression->Expressions.Add(FirstSequenceEntry);

				do
				{
					AST::FExpression* NextExpression = nullptr;
					if (ComputeExpr(Scanner, 1, /*Info,*/ SymbolScope, (ExpressionFlags), Allocator, &NextExpression, nullptr) != EParseResult::Matched)
					{
						Scanner.SourceError(TEXT("Expected expression!"));
						return ParseResultError();
					}
					AtomExpression->Expressions.Add(NextExpression);
				}
				while (Scanner.MatchToken(EHlslToken::Comma));
			}

			if (!Scanner.MatchToken(EHlslToken::RightParenthesis))
			{
				Scanner.SourceError(TEXT("Expected ')'!"));
				return ParseResultError();
			}
		}
			break;

		default:
		{
			AST::FTypeSpecifier* TypeSpecifier = nullptr;

			// Grrr handle Sampler as a variable name... This is safe here since Declarations are always handled first
			if (ParseGeneralType(Scanner, ETF_SAMPLER_TEXTURE_BUFFER, nullptr, Allocator, &TypeSpecifier) == EParseResult::Matched)
			{
				//@todo-rco: Check this var exists on the symbol table
				AtomExpression = new(Allocator) AST::FExpression(Allocator, AST::EOperators::Identifier, TypeSpecifier->SourceInfo);
				AtomExpression->Identifier = AST::FIdentifier::New(Allocator, TypeSpecifier->TypeName);
				break;
			}
			// Handle float3(x,y,z)
			else if (ParseGeneralType(Scanner, ETF_BUILTIN_NUMERIC, nullptr, Allocator, &TypeSpecifier) == EParseResult::Matched)
			{
				if (Scanner.MatchToken(EHlslToken::LeftParenthesis))
				{
					auto* TypeExpression = new(Allocator) AST::FExpression(Allocator, AST::EOperators::Identifier, TypeSpecifier->SourceInfo);
					TypeExpression->Identifier = AST::FIdentifier::New(Allocator, TypeSpecifier->TypeName);
					auto* FunctionCall = new(Allocator) AST::FFunctionExpression(Allocator, Token->SourceInfo, TypeExpression);
					auto Result = ParseExpressionList(EHlslToken::RightParenthesis, Scanner, SymbolScope, EHlslToken::Invalid, Allocator, FunctionCall);
					if (Result != EParseResult::Matched)
					{
						Scanner.SourceError(TEXT("Unexpected type in numeric constructor!"));
						return ParseResultError();
					}

					AtomExpression = FunctionCall;
				}
				else
				{
					Scanner.SourceError(TEXT("Unexpected type in declaration!"));
					return ParseResultError();
				}
				break;
			}
			else
			{
				if (UnaryResult == EParseResult::Matched)
				{
					Scanner.SourceError(TEXT("Expected expression!"));
					return ParseResultError();
				}

				return EParseResult::NotMatched;
			}
		}
			break;
		}

		check(AtomExpression);
		auto SuffixResult = MatchSuffixOperator(Scanner, /*Info,*/ SymbolScope, ExpressionFlags, Allocator, &AtomExpression, OutTernaryExpression);
		//auto* Token = Scanner.GetCurrentToken();
		if (/*!Token || */SuffixResult == EParseResult::Error)
		{
			return ParseResultError();
		}

		// Patch unary if necessary
		if (InnerUnaryExpression)
		{
			check(!InnerUnaryExpression->Expressions[0]);
			InnerUnaryExpression->Expressions[0] = AtomExpression;
		}

		if (!*OutExpression)
		{
			*OutExpression = AtomExpression;
		}

		return EParseResult::Matched;
	}

	int32 GetPrecedence(const FHlslToken* Token)
	{
		if (Token)
		{
			switch (Token->Token)
			{
			case EHlslToken::Comma:
				return 1;

			case EHlslToken::Equal:
			case EHlslToken::PlusEqual:
			case EHlslToken::MinusEqual:
			case EHlslToken::TimesEqual:
			case EHlslToken::DivEqual:
			case EHlslToken::ModEqual:
			case EHlslToken::GreaterGreaterEqual:
			case EHlslToken::LowerLowerEqual:
			case EHlslToken::AndEqual:
			case EHlslToken::OrEqual:
			case EHlslToken::XorEqual:
				return 2;

			case EHlslToken::Question:
				return 3;

			case EHlslToken::OrOr:
				return 4;

			case EHlslToken::AndAnd:
				return 5;

			case EHlslToken::Or:
				return 6;

			case EHlslToken::Xor:
				return 7;

			case EHlslToken::And:
				return 8;

			case EHlslToken::EqualEqual:
			case EHlslToken::NotEqual:
				return 9;

			case EHlslToken::Lower:
			case EHlslToken::Greater:
			case EHlslToken::LowerEqual:
			case EHlslToken::GreaterEqual:
				return 10;

			case EHlslToken::LowerLower:
			case EHlslToken::GreaterGreater:
				return 11;

			case EHlslToken::Plus:
			case EHlslToken::Minus:
				return 12;

			case EHlslToken::Times:
			case EHlslToken::Div:
			case EHlslToken::Mod:
				return 13;

			default:
				break;
			}
		}

		return -1;
	}

	bool IsTernaryOperator(const FHlslToken* Token)
	{
		return (Token && Token->Token == EHlslToken::Question);
	}

	bool IsBinaryOperator(const FHlslToken* Token)
	{
		return GetPrecedence(Token) > 0;
	}

	bool IsAssignmentOperator(const FHlslToken* Token)
	{
		if (Token)
		{
			switch (Token->Token)
			{
			case EHlslToken::Equal:
			case EHlslToken::PlusEqual:
			case EHlslToken::MinusEqual:
			case EHlslToken::TimesEqual:
			case EHlslToken::DivEqual:
			case EHlslToken::ModEqual:
			case EHlslToken::GreaterGreaterEqual:
			case EHlslToken::LowerLowerEqual:
			case EHlslToken::AndEqual:
			case EHlslToken::OrEqual:
			case EHlslToken::XorEqual:
				return true;

			default:
				break;
			}
		}

		return false;
	}

	static inline bool IsSequenceOperator(const FHlslToken* Token)
	{
		if (Token)
		{
			return (Token->Token == EHlslToken::Comma);
		}
		return false;
	}

	bool IsRightAssociative(const FHlslToken* Token)
	{
		return IsTernaryOperator(Token);
	}

	// Ternary is handled by popping out so it can right associate
	//#todo-rco: Fix the case for right-associative assignment operators
	EParseResult ComputeExpr(FHlslScanner& Scanner, int32 MinPrec, /*FInfo& Info,*/ FSymbolScope* SymbolScope, int32 ExpressionFlags, FLinearAllocator* Allocator, AST::FExpression** OutExpression, AST::FExpression** OutTernaryExpression)
	{
		const bool bAllowAssignment = ((ExpressionFlags & EEF_ALLOW_ASSIGNMENT) == EEF_ALLOW_ASSIGNMENT);

		auto OriginalToken = Scanner.GetCurrentTokenIndex();
		//FInfoIndentScope Scope(Info);
		/*
		// Precedence Climbing
		// http://eli.thegreenplace.net/2012/08/02/parsing-expressions-by-precedence-climbing
			compute_expr(min_prec):
			  result = compute_atom()

			  while cur token is a binary operator with precedence >= min_prec:
				prec, assoc = precedence and associativity of current token
				if assoc is left:
				  next_min_prec = prec + 1
				else:
				  next_min_prec = prec
				rhs = compute_expr(next_min_prec)
				result = compute operator(result, rhs)

			  return result
		*/
		//Info.PrintWithTabs(FString::Printf(TEXT("Compute Expr %d\n"), MinPrec));
		AST::FExpression* TernaryExpression = nullptr;
		auto Result = ComputeAtom(Scanner, /*Info,*/ SymbolScope, ExpressionFlags, Allocator, OutExpression, &TernaryExpression);
		if (Result != EParseResult::Matched)
		{
			return Result;
		}
		check(*OutExpression);
		do
		{
			const FHlslToken* Token = Scanner.GetCurrentToken();
			const int32 Precedence = GetPrecedence(Token);
			if (!Token || !IsBinaryOperator(Token) || Precedence < MinPrec || (!bAllowAssignment && IsAssignmentOperator(Token)) || IsSequenceOperator(Token) || (OutTernaryExpression && *OutTernaryExpression))
			{
				break;
			}

			Scanner.Advance();
			const int32 NextMinPrec = IsRightAssociative(Token) ? Precedence : Precedence + 1;
			AST::FExpression* RHSExpression = nullptr;
			AST::FExpression* RHSTernaryExpression = nullptr;
			Result = ComputeExpr(Scanner, NextMinPrec, /*Info,*/ SymbolScope, ExpressionFlags, Allocator, &RHSExpression, &RHSTernaryExpression);
			if (Result == EParseResult::Error)
			{
				return ParseResultError();
			}
			else if (Result == EParseResult::NotMatched)
			{
				break;
			}
			check(RHSExpression);
			const AST::EOperators BinaryOperator = AST::TokenToASTOperator(Token->Token);
			*OutExpression = new(Allocator) AST::FBinaryExpression(Allocator, BinaryOperator, *OutExpression, RHSExpression, Token->SourceInfo);
			if  (RHSTernaryExpression)
			{
				check(!TernaryExpression);
				TernaryExpression = RHSTernaryExpression;
				break;
			}
		}
		while (Scanner.HasMoreTokens());

		if (OriginalToken == Scanner.GetCurrentTokenIndex())
		{
			return EParseResult::NotMatched;
		}

		if (TernaryExpression)
		{
			if (!OutTernaryExpression)
			{
				if (!TernaryExpression->Expressions[0])
				{
					TernaryExpression->Expressions[0] = *OutExpression;
					*OutExpression = TernaryExpression;
				}
				else
				{
					check(0);
				}
			}
			else
			{
				*OutTernaryExpression = TernaryExpression;
			}
		}

		return EParseResult::Matched;
	}

	EParseResult ParseExpression(FHlslScanner& Scanner, FSymbolScope* SymbolScope, int32 ExpressionFlags, FLinearAllocator* Allocator, AST::FExpression** OutExpression)
	{
		/*FInfo Info(!true);*/
		return ComputeExpr(Scanner, 0, /*Info,*/ SymbolScope, ExpressionFlags, Allocator, OutExpression, nullptr);
	}

	EParseResult ParseExpressionList2(FHlslScanner& Scanner, FSymbolScope* SymbolScope, FLinearAllocator* Allocator, AST::FExpressionList::EType ExpressionType, AST::FExpression** OutExpression)
	{
		check(OutExpression);
		auto* Token = Scanner.GetCurrentToken();
		if (!Token)
		{
			Scanner.SourceError(TEXT("Invalid expression list\n"));
			return ParseResultError();
		}
		AST::FExpressionList* ExpressionList = new(Allocator) AST::FExpressionList(Allocator, ExpressionType, Token->SourceInfo);
		while (Scanner.HasMoreTokens())
		{
			auto* Peek = Scanner.PeekToken();
			if ((ExpressionType == AST::FExpressionList::EType::Braced && Peek->Token == EHlslToken::RightBrace) ||
				(ExpressionType == AST::FExpressionList::EType::Parenthesized && Peek->Token == EHlslToken::RightParenthesis))
			{
				*OutExpression = ExpressionList;
				return EParseResult::Matched;
			}

			AST::FExpression* Expression = nullptr;
			if (Scanner.MatchToken(EHlslToken::LeftBrace))
			{
				auto Result = ParseExpressionList2(Scanner, SymbolScope, Allocator, AST::FExpressionList::EType::Braced, &Expression);
				if (Result != EParseResult::Matched)
				{
					Scanner.SourceError(TEXT("Invalid expression list\n"));
					return ParseResultError();
				}

				if (!Scanner.MatchToken(EHlslToken::RightBrace))
				{
					Scanner.SourceError(TEXT("Invalid expression list; '}' expected\n"));
					return ParseResultError();
				}
			}
			else
			{
				auto Result = ParseExpression(Scanner, SymbolScope, EEF_ALLOW_ASSIGNMENT, Allocator, &Expression);
				if (Result == EParseResult::Error)
				{
					Scanner.SourceError(TEXT("Invalid expression list\n"));
					return ParseResultError();
				}
				else if (Result == EParseResult::NotMatched)
				{
					Scanner.SourceError(TEXT("Expected expression\n"));
					return ParseResultError();
				}
			}

			ExpressionList->Expressions.Add(Expression);

			if (!Scanner.MatchToken(EHlslToken::Comma))
			{
				*OutExpression = ExpressionList;
				return EParseResult::Matched;
			}
		}

		return EParseResult::NotMatched;
	}

	EParseResult ParseExpressionList(EHlslToken EndListToken, FHlslScanner& Scanner, FSymbolScope* SymbolScope, EHlslToken NewStartListToken, FLinearAllocator* Allocator, AST::FExpression* OutExpression)
	{
		check(OutExpression);
		while (Scanner.HasMoreTokens())
		{
			const auto* Token = Scanner.PeekToken();
			if (Token->Token == EndListToken)
			{
				Scanner.Advance();
				return EParseResult::Matched;
			}
			else if (NewStartListToken != EHlslToken::Invalid && Token->Token == NewStartListToken)
			{
				Scanner.Advance();
				checkNoEntry();
/*
				auto* SubExpression = new(Allocator) AST::FInitializerListExpression(Allocator, Token->SourceInfo);
				auto Result = ParseExpressionList(EndListToken, Scanner, SymbolScope, NewStartListToken, Allocator, SubExpression);
				if (Result != EParseResult::Matched)
				{
					return Result;
				}

				OutExpression->Expressions.Add(SubExpression);
*/
			}
			else
			{
				AST::FExpression* Expression = nullptr;
				auto Result = ParseExpression(Scanner, SymbolScope, EEF_ALLOW_ASSIGNMENT, Allocator, &Expression);
				if (Result == EParseResult::Error)
				{
					Scanner.SourceError(TEXT("Invalid expression list\n"));
					return ParseResultError();
				}
				else if (Result == EParseResult::NotMatched)
				{
					Scanner.SourceError(TEXT("Expected expression\n"));
					return ParseResultError();
				}

				OutExpression->Expressions.Add(Expression);
			}

			if (Scanner.MatchToken(EHlslToken::Comma))
			{
				continue;
			}
			else if (Scanner.MatchToken(EndListToken))
			{
				return EParseResult::Matched;
			}

			Scanner.SourceError(TEXT("Expected ','\n"));
			break;
		}

		return ParseResultError();
	}

	EParseResult ParseResultError()
	{
		// Extracted into a function so callstacks can be seen/debugged in the case of an error
		return EParseResult::Error;
	}
}
