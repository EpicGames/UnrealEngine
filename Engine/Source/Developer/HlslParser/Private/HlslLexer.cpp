// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HlslLexer.cpp - Implementation for scanning & tokenizing hlsl
=============================================================================*/

#include "HlslLexer.h"

namespace CrossCompiler
{
#define MATCH_TARGET(S) S, (UE_ARRAY_COUNT(S) - 1)

	typedef FPlatformTypes::TCHAR TCHAR;

	static FORCEINLINE bool IsSpaceOrTab(TCHAR Char)
	{
		return Char == ' ' || Char == '\t';
	}

	static FORCEINLINE bool IsEOL(TCHAR Char)
	{
		return Char == '\r' || Char == '\n';
	}

	static FORCEINLINE bool IsSpaceOrTabOrEOL(TCHAR Char)
	{
		return IsEOL(Char) || IsSpaceOrTab(Char);
	}

	static FORCEINLINE bool IsAlpha(TCHAR Char)
	{
		return (Char >= 'a' && Char <= 'z') || (Char >= 'A' && Char <= 'Z');
	}

	static FORCEINLINE bool IsDigit(TCHAR Char)
	{
		return Char >= '0' && Char <= '9';
	}

	static FORCEINLINE bool IsHexDigit(TCHAR Char)
	{
		return IsDigit(Char) || (Char >= 'a' && Char <= 'f') || (Char >= 'A' && Char <= 'F');
	}

	static FORCEINLINE bool IsAlphaOrDigit(TCHAR Char)
	{
		return IsAlpha(Char) || IsDigit(Char);
	}

	struct FKeywordToken
	{
		EHlslToken Current;
		void* Map;

		FKeywordToken() : Current(EHlslToken::Invalid), Map(nullptr) {}
	};
	typedef TMap<TCHAR, FKeywordToken> TCharKeywordTokenMap;
	TCharKeywordTokenMap Keywords;

	static void InsertToken(const TCHAR* String, EHlslToken Token)
	{
		TCharKeywordTokenMap* Map = &Keywords;
		while (*String)
		{
			FKeywordToken& KT = Map->FindOrAdd(*String);
			++String;
			if (!*String)
			{
				KT.Current = Token;
				return;
			}

			if (!KT.Map)
			{
				KT.Map = new TCharKeywordTokenMap();
			}

			Map = (TCharKeywordTokenMap*)KT.Map;
		}
	}

	static bool MatchSymbolToken(const TCHAR* InString, const TCHAR** OutString, EHlslToken& OutToken, TStringBuilderBase<TCHAR>* OutTokenString, bool bGreedy)
	{
		const TCHAR* OriginalString = InString;
		FKeywordToken* Found = Keywords.Find(*InString);

		if (OutString)
		{
			*OutString = OriginalString;
		}

		if (!Found)
		{
			return false;
		}

		do
		{
			++InString;
			if (Found->Map)
			{
				auto* Map = (TCharKeywordTokenMap*)Found->Map;
				FKeywordToken* NewFound = Map->Find(*InString);
				if (!NewFound)
				{
					if (Found->Current != EHlslToken::Invalid)
					{
						// Don't early out on a partial match (e.g., Texture1DSample should not be 2 tokens)
						if (!bGreedy || !*InString)
						{
							OutToken = Found->Current;
							if (OutTokenString)
							{
								OutTokenString->Reset();
								OutTokenString->Append(OriginalString, InString - OriginalString);
							}

							if (OutString)
							{
								*OutString = InString;
							}
							return true;
						}
					}

					return false;
				}
				Found = NewFound;
			}
			else if (bGreedy && *InString)
			{
				break;
			}
			else
			{
				OutToken = Found->Current;
				if (OutTokenString)
				{
					OutTokenString->Reset();
					OutTokenString->Append(OriginalString, InString - OriginalString);
				}

				if (OutString)
				{
					*OutString = InString;
				}
				return true;
			}
		}
		while (*InString);

		return false;
	}

	namespace Tokens
	{
		static struct FStaticInitializer
		{
			FStaticInitializer()
			{
				// Math
				InsertToken(TEXT("+"), EHlslToken::Plus);
				InsertToken(TEXT("+="), EHlslToken::PlusEqual);
				InsertToken(TEXT("-"), EHlslToken::Minus);
				InsertToken(TEXT("-="), EHlslToken::MinusEqual);
				InsertToken(TEXT("*"), EHlslToken::Times);
				InsertToken(TEXT("*="), EHlslToken::TimesEqual);
				InsertToken(TEXT("/"), EHlslToken::Div);
				InsertToken(TEXT("/="), EHlslToken::DivEqual);
				InsertToken(TEXT("%"), EHlslToken::Mod);
				InsertToken(TEXT("%="), EHlslToken::ModEqual);
				InsertToken(TEXT("("), EHlslToken::LeftParenthesis);
				InsertToken(TEXT(")"), EHlslToken::RightParenthesis);

				// Logical
				InsertToken(TEXT("=="), EHlslToken::EqualEqual);
				InsertToken(TEXT("!="), EHlslToken::NotEqual);
				InsertToken(TEXT("<"), EHlslToken::Lower);
				InsertToken(TEXT("<="), EHlslToken::LowerEqual);
				InsertToken(TEXT(">"), EHlslToken::Greater);
				InsertToken(TEXT(">="), EHlslToken::GreaterEqual);
				InsertToken(TEXT("&&"), EHlslToken::AndAnd);
				InsertToken(TEXT("||"), EHlslToken::OrOr);

				// Bit
				InsertToken(TEXT("<<"), EHlslToken::LowerLower);
				InsertToken(TEXT("<<="), EHlslToken::LowerLowerEqual);
				InsertToken(TEXT(">>"), EHlslToken::GreaterGreater);
				InsertToken(TEXT(">>="), EHlslToken::GreaterGreaterEqual);
				InsertToken(TEXT("&"), EHlslToken::And);
				InsertToken(TEXT("&="), EHlslToken::AndEqual);
				InsertToken(TEXT("|"), EHlslToken::Or);
				InsertToken(TEXT("|="), EHlslToken::OrEqual);
				InsertToken(TEXT("^"), EHlslToken::Xor);
				InsertToken(TEXT("^="), EHlslToken::XorEqual);
				InsertToken(TEXT("!"), EHlslToken::Not);
				InsertToken(TEXT("~"), EHlslToken::Neg);

				// Statements/Keywords
				InsertToken(TEXT("="), EHlslToken::Equal);
				InsertToken(TEXT("{"), EHlslToken::LeftBrace);
				InsertToken(TEXT("}"), EHlslToken::RightBrace);
				InsertToken(TEXT(";"), EHlslToken::Semicolon);
				InsertToken(TEXT("if"), EHlslToken::If);
				InsertToken(TEXT("else"), EHlslToken::Else);
				InsertToken(TEXT("for"), EHlslToken::For);
				InsertToken(TEXT("while"), EHlslToken::While);
				InsertToken(TEXT("do"), EHlslToken::Do);
				InsertToken(TEXT("return"), EHlslToken::Return);
				InsertToken(TEXT("switch"), EHlslToken::Switch);
				InsertToken(TEXT("case"), EHlslToken::Case);
				InsertToken(TEXT("break"), EHlslToken::Break);
				InsertToken(TEXT("default"), EHlslToken::Default);
				InsertToken(TEXT("continue"), EHlslToken::Continue);
				InsertToken(TEXT("goto"), EHlslToken::Goto);

				// Unary
				InsertToken(TEXT("++"), EHlslToken::PlusPlus);
				InsertToken(TEXT("--"), EHlslToken::MinusMinus);

				// Types
				InsertToken(TEXT("void"), EHlslToken::Void);
				InsertToken(TEXT("const"), EHlslToken::Const);
				InsertToken(TEXT("precise"), EHlslToken::Precise);

				InsertToken(TEXT("bool"), EHlslToken::Bool);
				InsertToken(TEXT("bool1"), EHlslToken::Bool1);
				InsertToken(TEXT("bool2"), EHlslToken::Bool2);
				InsertToken(TEXT("bool3"), EHlslToken::Bool3);
				InsertToken(TEXT("bool4"), EHlslToken::Bool4);
				InsertToken(TEXT("bool1x1"), EHlslToken::Bool1x1);
				InsertToken(TEXT("bool2x1"), EHlslToken::Bool2x1);
				InsertToken(TEXT("bool3x1"), EHlslToken::Bool3x1);
				InsertToken(TEXT("bool4x1"), EHlslToken::Bool4x1);
				InsertToken(TEXT("bool1x2"), EHlslToken::Bool1x2);
				InsertToken(TEXT("bool2x2"), EHlslToken::Bool2x2);
				InsertToken(TEXT("bool3x2"), EHlslToken::Bool3x2);
				InsertToken(TEXT("bool4x2"), EHlslToken::Bool4x2);
				InsertToken(TEXT("bool1x3"), EHlslToken::Bool1x3);
				InsertToken(TEXT("bool2x3"), EHlslToken::Bool2x3);
				InsertToken(TEXT("bool3x3"), EHlslToken::Bool3x3);
				InsertToken(TEXT("bool4x3"), EHlslToken::Bool4x3);
				InsertToken(TEXT("bool1x4"), EHlslToken::Bool1x4);
				InsertToken(TEXT("bool2x4"), EHlslToken::Bool2x4);
				InsertToken(TEXT("bool3x4"), EHlslToken::Bool3x4);
				InsertToken(TEXT("bool4x4"), EHlslToken::Bool4x4);

				InsertToken(TEXT("int"), EHlslToken::Int);
				InsertToken(TEXT("int1"), EHlslToken::Int1);
				InsertToken(TEXT("int2"), EHlslToken::Int2);
				InsertToken(TEXT("int3"), EHlslToken::Int3);
				InsertToken(TEXT("int4"), EHlslToken::Int4);
				InsertToken(TEXT("int1x1"), EHlslToken::Int1x1);
				InsertToken(TEXT("int2x1"), EHlslToken::Int2x1);
				InsertToken(TEXT("int3x1"), EHlslToken::Int3x1);
				InsertToken(TEXT("int4x1"), EHlslToken::Int4x1);
				InsertToken(TEXT("int1x2"), EHlslToken::Int1x2);
				InsertToken(TEXT("int2x2"), EHlslToken::Int2x2);
				InsertToken(TEXT("int3x2"), EHlslToken::Int3x2);
				InsertToken(TEXT("int4x2"), EHlslToken::Int4x2);
				InsertToken(TEXT("int1x3"), EHlslToken::Int1x3);
				InsertToken(TEXT("int2x3"), EHlslToken::Int2x3);
				InsertToken(TEXT("int3x3"), EHlslToken::Int3x3);
				InsertToken(TEXT("int4x3"), EHlslToken::Int4x3);
				InsertToken(TEXT("int1x4"), EHlslToken::Int1x4);
				InsertToken(TEXT("int2x4"), EHlslToken::Int2x4);
				InsertToken(TEXT("int3x4"), EHlslToken::Int3x4);
				InsertToken(TEXT("int4x4"), EHlslToken::Int4x4);

				InsertToken(TEXT("uint"), EHlslToken::Uint);
				InsertToken(TEXT("uint1"), EHlslToken::Uint1);
				InsertToken(TEXT("uint2"), EHlslToken::Uint2);
				InsertToken(TEXT("uint3"), EHlslToken::Uint3);
				InsertToken(TEXT("uint4"), EHlslToken::Uint4);
				InsertToken(TEXT("uint1x1"), EHlslToken::Uint1x1);
				InsertToken(TEXT("uint2x1"), EHlslToken::Uint2x1);
				InsertToken(TEXT("uint3x1"), EHlslToken::Uint3x1);
				InsertToken(TEXT("uint4x1"), EHlslToken::Uint4x1);
				InsertToken(TEXT("uint1x2"), EHlslToken::Uint1x2);
				InsertToken(TEXT("uint2x2"), EHlslToken::Uint2x2);
				InsertToken(TEXT("uint3x2"), EHlslToken::Uint3x2);
				InsertToken(TEXT("uint4x2"), EHlslToken::Uint4x2);
				InsertToken(TEXT("uint1x3"), EHlslToken::Uint1x3);
				InsertToken(TEXT("uint2x3"), EHlslToken::Uint2x3);
				InsertToken(TEXT("uint3x3"), EHlslToken::Uint3x3);
				InsertToken(TEXT("uint4x3"), EHlslToken::Uint4x3);
				InsertToken(TEXT("uint1x4"), EHlslToken::Uint1x4);
				InsertToken(TEXT("uint2x4"), EHlslToken::Uint2x4);
				InsertToken(TEXT("uint3x4"), EHlslToken::Uint3x4);
				InsertToken(TEXT("uint4x4"), EHlslToken::Uint4x4);

				InsertToken(TEXT("uint64_t"), EHlslToken::Uint64_t);
				InsertToken(TEXT("uint64_t1"), EHlslToken::Uint64_t1);
				InsertToken(TEXT("uint64_t2"), EHlslToken::Uint64_t2);
				InsertToken(TEXT("uint64_t3"), EHlslToken::Uint64_t3);
				InsertToken(TEXT("uint64_t4"), EHlslToken::Uint64_t4);
				InsertToken(TEXT("uint64_t1x1"), EHlslToken::Uint64_t1x1);
				InsertToken(TEXT("uint64_t2x1"), EHlslToken::Uint64_t2x1);
				InsertToken(TEXT("uint64_t3x1"), EHlslToken::Uint64_t3x1);
				InsertToken(TEXT("uint64_t4x1"), EHlslToken::Uint64_t4x1);
				InsertToken(TEXT("uint64_t1x2"), EHlslToken::Uint64_t1x2);
				InsertToken(TEXT("uint64_t2x2"), EHlslToken::Uint64_t2x2);
				InsertToken(TEXT("uint64_t3x2"), EHlslToken::Uint64_t3x2);
				InsertToken(TEXT("uint64_t4x2"), EHlslToken::Uint64_t4x2);
				InsertToken(TEXT("uint64_t1x3"), EHlslToken::Uint64_t1x3);
				InsertToken(TEXT("uint64_t2x3"), EHlslToken::Uint64_t2x3);
				InsertToken(TEXT("uint64_t3x3"), EHlslToken::Uint64_t3x3);
				InsertToken(TEXT("uint64_t4x3"), EHlslToken::Uint64_t4x3);
				InsertToken(TEXT("uint64_t1x4"), EHlslToken::Uint64_t1x4);
				InsertToken(TEXT("uint64_t2x4"), EHlslToken::Uint64_t2x4);
				InsertToken(TEXT("uint64_t3x4"), EHlslToken::Uint64_t3x4);
				InsertToken(TEXT("uint64_t4x4"), EHlslToken::Uint64_t4x4);

				InsertToken(TEXT("ulong"), EHlslToken::Uint64_t);	
				InsertToken(TEXT("ulong2"), EHlslToken::Uint64_t2);	
				InsertToken(TEXT("ulong3"), EHlslToken::Uint64_t3);	
				InsertToken(TEXT("ulong4"), EHlslToken::Uint64_t4);	

				InsertToken(TEXT("half"), EHlslToken::Half);
				InsertToken(TEXT("half1"), EHlslToken::Half1);
				InsertToken(TEXT("half2"), EHlslToken::Half2);
				InsertToken(TEXT("half3"), EHlslToken::Half3);
				InsertToken(TEXT("half4"), EHlslToken::Half4);
				InsertToken(TEXT("half1x1"), EHlslToken::Half1x1);
				InsertToken(TEXT("half2x1"), EHlslToken::Half2x1);
				InsertToken(TEXT("half3x1"), EHlslToken::Half3x1);
				InsertToken(TEXT("half4x1"), EHlslToken::Half4x1);
				InsertToken(TEXT("half1x2"), EHlslToken::Half1x2);
				InsertToken(TEXT("half2x2"), EHlslToken::Half2x2);
				InsertToken(TEXT("half3x2"), EHlslToken::Half3x2);
				InsertToken(TEXT("half4x2"), EHlslToken::Half4x2);
				InsertToken(TEXT("half1x3"), EHlslToken::Half1x3);
				InsertToken(TEXT("half2x3"), EHlslToken::Half2x3);
				InsertToken(TEXT("half3x3"), EHlslToken::Half3x3);
				InsertToken(TEXT("half4x3"), EHlslToken::Half4x3);
				InsertToken(TEXT("half1x4"), EHlslToken::Half1x4);
				InsertToken(TEXT("half2x4"), EHlslToken::Half2x4);
				InsertToken(TEXT("half3x4"), EHlslToken::Half3x4);
				InsertToken(TEXT("half4x4"), EHlslToken::Half4x4);

				InsertToken(TEXT("min16float"), EHlslToken::Min16Float);
				InsertToken(TEXT("min16float1"), EHlslToken::Min16Float1);
				InsertToken(TEXT("min16float2"), EHlslToken::Min16Float2);
				InsertToken(TEXT("min16float3"), EHlslToken::Min16Float3);
				InsertToken(TEXT("min16float4"), EHlslToken::Min16Float4);
				InsertToken(TEXT("min16float1x1"), EHlslToken::Min16Float1x1);
				InsertToken(TEXT("min16float2x1"), EHlslToken::Min16Float2x1);
				InsertToken(TEXT("min16float3x1"), EHlslToken::Min16Float3x1);
				InsertToken(TEXT("min16float4x1"), EHlslToken::Min16Float4x1);
				InsertToken(TEXT("min16float1x2"), EHlslToken::Min16Float1x2);
				InsertToken(TEXT("min16float2x2"), EHlslToken::Min16Float2x2);
				InsertToken(TEXT("min16float3x2"), EHlslToken::Min16Float3x2);
				InsertToken(TEXT("min16float4x2"), EHlslToken::Min16Float4x2);
				InsertToken(TEXT("min16float1x3"), EHlslToken::Min16Float1x3);
				InsertToken(TEXT("min16float2x3"), EHlslToken::Min16Float2x3);
				InsertToken(TEXT("min16float3x3"), EHlslToken::Min16Float3x3);
				InsertToken(TEXT("min16float4x3"), EHlslToken::Min16Float4x3);
				InsertToken(TEXT("min16float1x4"), EHlslToken::Min16Float1x4);
				InsertToken(TEXT("min16float2x4"), EHlslToken::Min16Float2x4);
				InsertToken(TEXT("min16float3x4"), EHlslToken::Min16Float3x4);
				InsertToken(TEXT("min16float4x4"), EHlslToken::Min16Float4x4);

				InsertToken(TEXT("float"), EHlslToken::Float);
				InsertToken(TEXT("float1"), EHlslToken::Float1);
				InsertToken(TEXT("float2"), EHlslToken::Float2);
				InsertToken(TEXT("float3"), EHlslToken::Float3);
				InsertToken(TEXT("float4"), EHlslToken::Float4);
				InsertToken(TEXT("float1x1"), EHlslToken::Float1x1);
				InsertToken(TEXT("float2x1"), EHlslToken::Float2x1);
				InsertToken(TEXT("float3x1"), EHlslToken::Float3x1);
				InsertToken(TEXT("float4x1"), EHlslToken::Float4x1);
				InsertToken(TEXT("float1x2"), EHlslToken::Float1x2);
				InsertToken(TEXT("float2x2"), EHlslToken::Float2x2);
				InsertToken(TEXT("float3x2"), EHlslToken::Float3x2);
				InsertToken(TEXT("float4x2"), EHlslToken::Float4x2);
				InsertToken(TEXT("float1x3"), EHlslToken::Float1x3);
				InsertToken(TEXT("float2x3"), EHlslToken::Float2x3);
				InsertToken(TEXT("float3x3"), EHlslToken::Float3x3);
				InsertToken(TEXT("float4x3"), EHlslToken::Float4x3);
				InsertToken(TEXT("float1x4"), EHlslToken::Float1x4);
				InsertToken(TEXT("float2x4"), EHlslToken::Float2x4);
				InsertToken(TEXT("float3x4"), EHlslToken::Float3x4);
				InsertToken(TEXT("float4x4"), EHlslToken::Float4x4);

				InsertToken(TEXT("texture"), EHlslToken::Texture);
				InsertToken(TEXT("Texture1D"), EHlslToken::Texture1D);
				InsertToken(TEXT("Texture1DArray"), EHlslToken::Texture1DArray);
				InsertToken(TEXT("Texture1D_Array"), EHlslToken::Texture1DArray);	// PSSL
				InsertToken(TEXT("Texture2D"), EHlslToken::Texture2D);
				InsertToken(TEXT("Texture2DArray"), EHlslToken::Texture2DArray);
				InsertToken(TEXT("Texture2D_Array"), EHlslToken::Texture2DArray);	// PSSL
				InsertToken(TEXT("Texture2DMS"), EHlslToken::Texture2DMS);
				InsertToken(TEXT("MS_Texture2D"), EHlslToken::Texture2DMS);	// PSSL
				InsertToken(TEXT("Texture2DMSArray"), EHlslToken::Texture2DMSArray);
				InsertToken(TEXT("MS_Texture2D_Array"), EHlslToken::Texture2DMS);	// PSSL
				InsertToken(TEXT("Texture3D"), EHlslToken::Texture3D);
				InsertToken(TEXT("TextureCube"), EHlslToken::TextureCube);
				InsertToken(TEXT("TextureCubeArray"), EHlslToken::TextureCubeArray);
				InsertToken(TEXT("TextureCube_Array"), EHlslToken::TextureCubeArray);	// PSSL

				InsertToken(TEXT("sampler"), EHlslToken::Sampler);
				InsertToken(TEXT("sampler1D"), EHlslToken::Sampler1D);
				InsertToken(TEXT("sampler2D"), EHlslToken::Sampler2D);
				InsertToken(TEXT("sampler3D"), EHlslToken::Sampler3D);
				InsertToken(TEXT("samplerCUBE"), EHlslToken::SamplerCube);
				InsertToken(TEXT("SamplerState"), EHlslToken::SamplerState);
				InsertToken(TEXT("SamplerComparisonState"), EHlslToken::SamplerComparisonState);

				InsertToken(TEXT("Buffer"), EHlslToken::Buffer);
				InsertToken(TEXT("DataBuffer"), EHlslToken::Buffer);	// PSSL
				InsertToken(TEXT("AppendStructuredBuffer"), EHlslToken::AppendStructuredBuffer);
				InsertToken(TEXT("AppendRegularBuffer"), EHlslToken::AppendStructuredBuffer);	// PSSL
				InsertToken(TEXT("ByteAddressBuffer"), EHlslToken::ByteAddressBuffer);
				InsertToken(TEXT("ByteBuffer"), EHlslToken::ByteAddressBuffer);	// PSSL
				InsertToken(TEXT("ConsumeStructuredBuffer"), EHlslToken::ConsumeStructuredBuffer);
				InsertToken(TEXT("ConsumeRegularBuffer"), EHlslToken::ConsumeStructuredBuffer);	// PSSL
				InsertToken(TEXT("RWBuffer"), EHlslToken::RWBuffer);
				InsertToken(TEXT("RW_DataBuffer"), EHlslToken::RWBuffer);	// PSSL
				InsertToken(TEXT("RWByteAddressBuffer"), EHlslToken::RWByteAddressBuffer);
				InsertToken(TEXT("RW_ByteBuffer"), EHlslToken::RWByteAddressBuffer);	// PSSL
				InsertToken(TEXT("RWStructuredBuffer"), EHlslToken::RWStructuredBuffer);
				InsertToken(TEXT("RW_RegularBuffer"), EHlslToken::RWStructuredBuffer);	// PSSL
				InsertToken(TEXT("RWTexture1D"), EHlslToken::RWTexture1D);
				InsertToken(TEXT("RW_Texture1D"), EHlslToken::RWTexture1D);	// PSSL
				InsertToken(TEXT("RWTexture1DArray"), EHlslToken::RWTexture1DArray);
				InsertToken(TEXT("RW_Texture1D_Array"), EHlslToken::RWTexture1DArray);	// PSSL
				InsertToken(TEXT("RWTexture2D"), EHlslToken::RWTexture2D);
				InsertToken(TEXT("RW_Texture2D"), EHlslToken::RWTexture2D);	// PSSL
				InsertToken(TEXT("RWTexture2DArray"), EHlslToken::RWTexture2DArray);
				InsertToken(TEXT("RW_Texture2D_Array"), EHlslToken::RWTexture2DArray);	// PSSL
				InsertToken(TEXT("RasterizerOrderedTexture2D"), EHlslToken::RasterizerOrderedTexture2D);
				InsertToken(TEXT("RWTexture3D"), EHlslToken::RWTexture3D);
				InsertToken(TEXT("RW_Texture3D"), EHlslToken::RWTexture3D);	// PSSL
				InsertToken(TEXT("StructuredBuffer"), EHlslToken::StructuredBuffer);
				InsertToken(TEXT("RegularBuffer"), EHlslToken::StructuredBuffer);	// PSSL
				InsertToken(TEXT("ConstantBuffer"), EHlslToken::ConstantBuffer);
				InsertToken(TEXT("RaytracingAccelerationStructure"), EHlslToken::RaytracingAccelerationStructure);

				// Modifiers
				InsertToken(TEXT("in"), EHlslToken::In);
				InsertToken(TEXT("out"), EHlslToken::Out);
				InsertToken(TEXT("inout"), EHlslToken::InOut);
				InsertToken(TEXT("static"), EHlslToken::Static);
				InsertToken(TEXT("uniform"), EHlslToken::Uniform);

				// Misc
				InsertToken(TEXT("["), EHlslToken::LeftSquareBracket);
				InsertToken(TEXT("]"), EHlslToken::RightSquareBracket);
				InsertToken(TEXT("?"), EHlslToken::Question);
				InsertToken(TEXT("::"), EHlslToken::ColonColon);
				InsertToken(TEXT(":"), EHlslToken::Colon);
				InsertToken(TEXT(","), EHlslToken::Comma);
				InsertToken(TEXT("."), EHlslToken::Dot);
				InsertToken(TEXT("struct"), EHlslToken::Struct);
				InsertToken(TEXT("class"), EHlslToken::Struct); // Scan as equivalent to "struct"
				InsertToken(TEXT("cbuffer"), EHlslToken::CBuffer);
				InsertToken(TEXT("ConstantBuffer"), EHlslToken::ConstantBuffer);
				InsertToken(TEXT("groupshared"), EHlslToken::GroupShared);
				InsertToken(TEXT("row_major"), EHlslToken::RowMajor);
				InsertToken(TEXT("register"), EHlslToken::Register);
				InsertToken(TEXT("inline"), EHlslToken::Inline);
				InsertToken(TEXT("typedef"), EHlslToken::Typedef);
				InsertToken(TEXT("packoffset"), EHlslToken::PackOffset);
				InsertToken(TEXT("namespace"), EHlslToken::Namespace);
				InsertToken(TEXT("operator"), EHlslToken::Operator);
				InsertToken(TEXT("_Static_assert"), EHlslToken::StaticAssert); // HLSL2021 adopted C11 '_Static_assert'-statements
				InsertToken(TEXT("static_assert"), EHlslToken::StaticAssert); // Some shader compilers support C++11 'static_assert'-statements
				InsertToken(TEXT("_Pragma"), EHlslToken::C99Pragma); // C99/C++11 style pragma (distinct from #pragma because argument is a string constant)
			}
		} GStaticInitializer;
	}

	struct FTokenizer
	{
		FString Filename;
		const TCHAR* Current;
		const TCHAR* End;
		const TCHAR* CurrentLineStart;
		int32 Line;

		FTokenizer(const FString& InString, const FString& InFilename = TEXT("")) :
			Filename(InFilename),
			Current(nullptr),
			End(nullptr),
			CurrentLineStart(nullptr),
			Line(0)
		{
			if (InString.Len() > 0)
			{
				Current = *InString;
				End = *InString + InString.Len();
				Line = 1;
				CurrentLineStart = Current;
			}
		}

		bool HasCharsAvailable() const
		{
			return Current < End;
		}

		void SkipWhitespaceInLine()
		{
			while (HasCharsAvailable())
			{
				auto Char = Peek();
				if (!IsSpaceOrTab(Char))
				{
					break;
				}

				++Current;
			}
		}

		void SkipWhitespaceAndEmptyLines()
		{
			while (HasCharsAvailable())
			{
				SkipWhitespaceInLine();
				auto Char = Peek();
				if (IsEOL(Char))
				{
					SkipToNextLine();
				}
				else
				{
					auto NextChar = Peek(1);
					if (Char == '/' && NextChar == '/')
					{
						// C++ comment
						Current += 2;
						this->SkipToNextLine();
						continue;
					}
					else if (Char == '/' && NextChar == '*')
					{
						// C Style comment, eat everything up to * /
						Current += 2;
						bool bClosedComment = false;
						while (HasCharsAvailable())
						{
							if (Peek() == '*')
							{
								if (Peek(1) == '/')
								{
									bClosedComment = true;
									Current += 2;
									break;
								}

							}
							else if (Peek() == '\n')
							{
								SkipToNextLine();

								// Don't increment current!
								continue;
							}

							++Current;
						}
						//@todo-rco: Error if no closing * / found and we got to EOL
						//check(bClosedComment);
					}
					else
					{
						break;
					}
				}
			}
		}

		TCHAR Peek() const
		{
			if (HasCharsAvailable())
			{
				return *Current;
			}

			return 0;
		}

		TCHAR Peek(int32 Delta) const
		{
			check(Delta > 0);
			if (Current + Delta < End)
			{
				return Current[Delta];
			}

			return 0;
		}

		void SkipToNextLine()
		{
			while (HasCharsAvailable())
			{
				auto Char = Peek();
				++Current;
				if (Char == '\r' && Peek() == '\n')
				{
					++Current;
					break;
				}
				else if (Char == '\n')
				{
					break;
				}
			}

			++Line;
			CurrentLineStart = Current;
		}

		bool MatchString(FStringView String)
		{
			return MatchString(String.GetData(), String.Len());
		}

		bool MatchString(const TCHAR* Target, int32 TargetLen)
		{
			if (Current + TargetLen <= End)
			{
				if (FCString::Strncmp(Current, Target, TargetLen) == 0)
				{
					Current += TargetLen;
					return true;
				}
			}
			return false;
		}

		bool PeekDigit() const
		{
			return IsDigit(Peek());
		}

		bool MatchAndSkipDigits()
		{
			auto* Original = Current;
			while (PeekDigit())
			{
				++Current;
			}

			return Original != Current;
		}

		bool Match(TCHAR Char)
		{
			if (Char == Peek())
			{
				++Current;
				return true;
			}

			return false;
		}

		inline bool IsSwizzleDigit(TCHAR Char)
		{
			switch (Char)
			{
			case 'r':
			case 'g':
			case 'b':
			case 'a':
			case 'x':
			case 'y':
			case 'z':
			case 'w':
				return true;

			default:
				return false;
			}
		}

		bool MatchLiteralFloat(TStringBuilderBase<TCHAR>& OutLiteral, ELiteralType& OutType)
		{
			auto* Original = Current;
			TCHAR Char = Peek();
			ELiteralType Type = ELiteralType::Float;

			// \.[0-9]+([eE][+-]?[0-9]+)?[fF]?			-> Dot Digits+ Exp? F?
			// [0-9]+\.([eE][+-]?[0-9]+)?[fF]?			-> Digits+ Dot Exp? F?
			// [0-9]+\.[0-9]+([eE][+-]?[0-9]+)?[fF]?	-> Digits+ Dot Digits+ Exp? F?
			// [0-9]+[eE][+-]?[0-9]+[fF]?				-> Digits+ Exp F?
			// [0-9]+[fF]								-> Digits+ F
			// 1.#INF									-> Infinity constant
			if (!IsDigit(Char) && Char != '.')
			{
				return false;
			}

			bool bExpOptional = false;

			// Differentiate between 1. and 1.rr for example
			if (Char == '.' && IsSwizzleDigit(Peek(1)))
			{
				goto NotFloat;
			}

			if (Match('.') && MatchAndSkipDigits())
			{
				bExpOptional = true;
			}
			else if (MatchAndSkipDigits())
			{
				// Differentiate between 1. and 1.rr for example
				if (Peek() == '.' && IsSwizzleDigit(Peek(1)))
				{
					goto NotFloat;
				}

				if (Match('.'))
				{
					// Check for infinity constant
					if (Match('#'))
					{
						if (MatchString(MATCH_TARGET(TEXT("INF"))))
						{
							goto Done;
						}
						else
						{
							goto NotFloat;
						}
					}

					bExpOptional = true;
					MatchAndSkipDigits();
				}
				else
				{
					if (IsValidFloatSuffix(Char))
					{
						Type = ELiteralType::FloatSuffix;
						goto Done;
					}

					bExpOptional = false;
				}
			}
			else
			{
				goto NotFloat;
			}

			{
				// Exponent [eE][+-]?[0-9]+
				bool bExponentFound = false;
				if (Match('e') || Match('E'))
				{
					Char = Peek();
					if (Char == '+' || Char == '-')
					{
						++Current;
					}

					if (MatchAndSkipDigits())
					{
						bExponentFound = true;
					}
				}

				if (!bExponentFound && !bExpOptional)
				{
					goto NotFloat;
				}
			}

			// [fF]
			Char = Peek();
			if (IsValidFloatSuffix(Char))
			{
				++Current;
				Type = ELiteralType::FloatSuffix;
			}

		Done:
			OutLiteral.Append(Original, static_cast<int32>(Current - Original));
			OutType = Type;
			return true;

		NotFloat:
			Current = Original;
			return false;
		}

		bool MatchQuotedString(TStringBuilderBase<TCHAR>& OutString)
		{
			if (!Match('"'))
			{
				return false;
			}

			OutString = TEXT("");
			while (Peek() != '"')
			{
				auto Char = Peek();
				OutString += Char;
				if (Char == 0)
				{
					return false; // ill-formed string (EOL inside quote)
				}
				if (Char == '\\') // escaped character, we must have at least one more
				{
					++Current;
					Char = Peek();
					if (Char == 0)
					{
						return false; // EOL
					}
					OutString += Char; //@todo-rco: Should we validate we have escaped a valid character?
				}
				++Current;
			}

			if (Match('"'))
			{
				return true;
			}

			//@todo-rco: Error!
			check(0);
			return false;
		}

		bool MatchIdentifier(TStringBuilderBase<TCHAR>& OutIdentifier)
		{
			if (HasCharsAvailable())
			{
				auto Char = Peek();
				if (!IsAlpha(Char) && Char != '_')
				{
					return false;
				}

				++Current;
				OutIdentifier = TEXT("");
				OutIdentifier += Char;
				do
				{
					Char = Peek();
					if (!IsAlphaOrDigit(Char) && Char != '_')
					{
						break;
					}
					OutIdentifier += Char;
					++Current;
				}
				while (HasCharsAvailable());
				return true;
			}

			return false;
		}

		bool MatchSymbol(EHlslToken& OutToken, TStringBuilderBase<TCHAR>& OutTokenString)
		{
			if (HasCharsAvailable())
			{
				if (MatchSymbolToken(Current, &Current, OutToken, &OutTokenString, false))
				{
					return true;
				}
			}

			return false;
		}

		static void ProcessDirective(FTokenizer& Tokenizer, FCompilerMessages& CompilerMessages, class FHlslScanner& Scanner);

		void ReadToEndOfLine(TStringBuilderBase<TCHAR>& OutString, bool bSkipToNextLine = true)
		{
			const TCHAR* Start = Current;
			const TCHAR* EndOfLine = Current;

			while (HasCharsAvailable())
			{
				auto Char = Peek();
				if (Char == '\r' && Peek() == '\n')
				{
					break;
				}
				else if (Char == '\n')
				{
					break;
				}
				else
				{
					EndOfLine = Current;
					++Current;
				}
			}

			if (bSkipToNextLine)
			{
				SkipToNextLine();
			}

			int32 Count = (int32)(EndOfLine - Start) + 1;
			OutString.Append(Start, Count);
		}

		bool RuleDecimalInteger(TStringBuilderBase<TCHAR>& OutValue)
		{
			// [1-9][0-9]*
			auto Char = Peek();
			if (Char < '1' || Char > '9')
			{
				return false;
			}
			OutValue = TEXT("");
			++Current;
			OutValue += Char;

			while (HasCharsAvailable())
			{
				Char = Peek();
				if (!IsDigit(Char))
				{
					break;
				}
				++Current;
				OutValue += Char;
			}

			return true;
		}

		bool RuleDecimalIntegerValue(uint32& OutValue)
		{
			TStringBuilder<11> String; // large enough for uint32 max + terminator
			if (RuleDecimalInteger(String))
			{
				OutValue = (uint32)FCString::Atoi(*String);
				return true;
			}

			return false;
		}

		bool RuleOctalInteger(TStringBuilderBase<TCHAR>& OutValue)
		{
			// 0[0-7]*
			auto Char = Peek();
			if (Char != '0')
			{
				return false;
			}

			OutValue = TEXT("0");
			++Current;
			while (HasCharsAvailable())
			{
				Char = Peek();
				if (Char >= '0' && Char <= '7')
				{
					OutValue += Char;
				}
				else
				{
					break;
				}
				++Current;
			}

			return true;
		}

		bool RuleHexadecimalInteger(TStringBuilderBase<TCHAR>& OutValue)
		{
			// 0[xX][0-9a-zA-Z]+
			auto Char = Peek();
			auto Char1 = Peek(1);
			auto Char2 = Peek(2);
			if (Char == '0' && (Char1 == 'x' || Char1 == 'X') && IsHexDigit(Char2))
			{
				Current += 2;
				OutValue = TEXT("0");
				OutValue += Char1;
				do
				{
					Char = Peek();
					if (IsDigit(Char) || (Char >= 'a' && Char <= 'f') || (Char >= 'A' && Char <= 'F'))
					{
						OutValue += Char;
					}
					else
					{
						break;
					}
					++Current;
				}
				while (HasCharsAvailable());

				return true;
			}

			return false;
		}

		static bool IsValidIntegerSuffix(TCHAR Char)
		{
			switch (Char)
			{
			case (TCHAR)'u':
			case (TCHAR)'U':
			case (TCHAR)'l':
			case (TCHAR)'L':
				return true;
			default:
				break;
			}

			return false;
		}

		static bool IsValidFloatSuffix(TCHAR Char)
		{
			switch (Char)
			{
			case (TCHAR)'F':
			case (TCHAR)'f':
				return true;
			default:
				break;
			}

			return false;
		}

		bool MatchLiteral(TStringBuilderBase<TCHAR>& OutValue, ELiteralType& OutLiteralType)
		{
			if (MatchLiteralFloat(OutValue, OutLiteralType))
			{
				return true;
			}
			else
			{
				return MatchLiteralInteger(OutValue, OutLiteralType);
			}
		}

		bool MatchLiteralInteger(TStringBuilderBase<TCHAR>& OutValue, ELiteralType& OutLiteralType)
		{
			if (RuleHexadecimalInteger(OutValue))
			{
				auto Char = Peek();
				if (IsValidIntegerSuffix(Char))
				{
					++Current;
					OutValue += Char;
					OutLiteralType = ELiteralType::HexSuffix;
				}
				else
				{
					OutLiteralType = ELiteralType::Hex;
				}
				return true;
			}
			else if (RuleOctalInteger(OutValue))
			{
				if (OutValue.Len() == 1 && OutValue.LastChar() == (TCHAR)'0')
				{
					auto Char = Peek();
					if (IsValidIntegerSuffix(Char))
					{
						++Current;
						OutValue += Char;
						OutLiteralType = ELiteralType::IntegerSuffix;
					}
					else
					{
						OutLiteralType = ELiteralType::Integer;
					}
				}
				else
				{
					OutLiteralType = ELiteralType::Octal;
				}
				return true;
			}
			else if (RuleDecimalInteger(OutValue))
			{
				auto Char = Peek();
				if (IsValidIntegerSuffix(Char))
				{
					++Current;
					OutValue += Char;
					OutLiteralType = ELiteralType::IntegerSuffix;
				}
				else
				{
					OutLiteralType = ELiteralType::Integer;
				}
				return true;
			}

			return false;
		}
	};

	FHlslScanner::FHlslScanner(FCompilerMessages& InCompilerMessages) :
		CompilerMessages(InCompilerMessages),
		CurrentToken(0)
	{
	}

	FHlslScanner::~FHlslScanner()
	{
	}

	inline void FHlslScanner::EmplaceToken(FHlslToken&& Token, const FTokenizer& Tokenizer)
	{
		int32 TokenIndex = Tokens.Emplace(MoveTemp(Token));
		Tokens[TokenIndex].SourceInfo.Filename = &SourceFilenames.Last();
		Tokens[TokenIndex].SourceInfo.Line = Tokenizer.Line;
		Tokens[TokenIndex].SourceInfo.Column = (int32)(Tokenizer.Current - Tokenizer.CurrentLineStart) + 1;
	}

	void FHlslScanner::Clear(const FString& Filename)
	{
		Tokens.Empty();
		SourceFilenames.Empty();
		SourceFilenames.Add(new FString(Filename));
	}

	bool FHlslScanner::Lex(const FString& String, const FString& Filename)
	{
		Clear(Filename);
		
		// Simple heuristic to avoid reallocating
		Tokens.Reserve(String.Len() / 8);

		FTokenizer Tokenizer(String, Filename);
		while (Tokenizer.HasCharsAvailable())
		{
			auto* Sanity = Tokenizer.Current;
			Tokenizer.SkipWhitespaceAndEmptyLines();
			if (Tokenizer.Peek() == '#')
			{
				FTokenizer::ProcessDirective(Tokenizer, CompilerMessages, *this);
				if (Tokenizer.Filename != SourceFilenames.Last())
				{
					SourceFilenames.Add(new FString(Tokenizer.Filename));
				}
			}
			else
			{
				TStringBuilder<128> Identifier;
				EHlslToken SymbolToken;
				ELiteralType LiteralType = ELiteralType::Unknown;
				if (Tokenizer.MatchLiteral(Identifier, LiteralType))
				{
					EmplaceToken(FHlslToken(Identifier, LiteralType), Tokenizer);
				}
				else if (Tokenizer.MatchIdentifier(Identifier))
				{
					if (!FCString::Strcmp(*Identifier, TEXT("true")) || !FCString::Strcmp(*Identifier, TEXT("false")))
					{
						EmplaceToken(FHlslToken(*Identifier, ELiteralType::Bool), Tokenizer);
					}
					else if (MatchSymbolToken(*Identifier, nullptr, SymbolToken, nullptr, true))
					{
						EmplaceToken(FHlslToken(SymbolToken, Identifier), Tokenizer);
					}
					else
					{
						EmplaceToken(FHlslToken(Identifier), Tokenizer);
					}
				}
				else if (Tokenizer.MatchSymbol(SymbolToken, Identifier))
				{
					EmplaceToken(FHlslToken(SymbolToken, Identifier), Tokenizer);
				}
				else if (Tokenizer.MatchQuotedString(Identifier))
				{
					EmplaceToken(FHlslToken(EHlslToken::StringConstant, Identifier), Tokenizer);
				}
				else if (Tokenizer.HasCharsAvailable())
				{
					//@todo-rco: Unknown token!
					if (Tokenizer.Filename.Len() > 0)
					{
						CompilerMessages.SourceError(*FString::Printf(TEXT("Unknown token at line %d, file '%s'!"), Tokenizer.Line, *Tokenizer.Filename));
					}
					else
					{
						CompilerMessages.SourceError(*FString::Printf(TEXT("Unknown token at line %d!"), Tokenizer.Line));
					}
					return false;
				}
			}

			check(Sanity != Tokenizer.Current);
		}

		return true;
	}

	void FHlslScanner::Dump()
	{
		for (int32 Index = 0; Index < Tokens.Num(); ++Index)
		{
			auto& Token = Tokens[Index];
			switch (Token.Token)
			{
			case EHlslToken::Literal:
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("** %d: Literal Type %d '%s'\n"), Index, (int32)Token.LiteralType, *Token.String);
				break;

			default:
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("** %d: %d '%s'\n"), Index, Token.Token, *Token.String);
				break;
			}
		}
	}

	bool FHlslScanner::MatchToken(EHlslToken InToken)
	{
		const auto* Token = GetCurrentToken();
		if (Token)
		{
			if (Token->Token == InToken)
			{
				++CurrentToken;
				return true;
			}
		}

		return false;
	}

	bool FHlslScanner::MatchIntegerLiteral()
	{
		const auto* Token = GetCurrentToken();
		if (Token)
		{
			if (Token->Token == EHlslToken::Literal)
			{
				if (IsIntegerType(Token->LiteralType))
				{
					++CurrentToken;
					return true;
				}
			}
		}

		return false;
	}

	const FHlslToken* FHlslScanner::PeekToken(uint32 LookAhead /*= 0*/) const
	{
		if (CurrentToken + LookAhead < (uint32)Tokens.Num())
		{
			return &Tokens[CurrentToken + LookAhead];
		}

		return nullptr;
	}

	bool FHlslScanner::HasMoreTokens() const
	{
		return CurrentToken < (uint32)Tokens.Num();
	}

	const FHlslToken* FHlslScanner::GetCurrentToken() const
	{
		if (CurrentToken < (uint32)Tokens.Num())
		{
			return &Tokens[CurrentToken];
		}
		return nullptr;
	}

	void FHlslScanner::SetCurrentTokenIndex(uint32 NewToken)
	{
		check(NewToken <= (uint32)Tokens.Num());
		CurrentToken = NewToken;
	}

	void FHlslScanner::SourceError(const FString& Error)
	{
		if (CurrentToken < (uint32)Tokens.Num())
		{
			const FHlslToken& Token = Tokens[CurrentToken];
			check(Token.SourceInfo.Filename);
			CompilerMessages.SourceError(Token.SourceInfo, *Error);
		}
		else
		{
			CompilerMessages.SourceError(*Error);
		}
	}

	void FTokenizer::ProcessDirective(FTokenizer& Tokenizer, FCompilerMessages& CompilerMessages, FHlslScanner& Scanner)
	{
		check(Tokenizer.Peek() == TEXT('#'));
		if (Tokenizer.MatchString(MATCH_TARGET(TEXT("#line"))))
		{
			Tokenizer.SkipWhitespaceInLine();
			uint32 Line = 0;
			if (Tokenizer.RuleDecimalIntegerValue(Line))
			{
				Tokenizer.Line = Line - 1;
				Tokenizer.SkipWhitespaceInLine();
				TStringBuilder<256> Filename;
				if (Tokenizer.MatchQuotedString(Filename))
				{
					Tokenizer.Filename = Filename;
				}
			}
			else
			{
				TStringBuilder<128> LineString;
				LineString += TEXTVIEW("#line ");
				Tokenizer.ReadToEndOfLine(LineString);

				CompilerMessages.SourceError(*FString::Printf(TEXT("Malformed #line directive: %s!"), *LineString));
			}
		}
		else if (Tokenizer.MatchString(MATCH_TARGET(TEXT("#pragma"))))
		{
			TStringBuilder<128> Pragma;
			Pragma += TEXTVIEW("#pragma");
			Tokenizer.ReadToEndOfLine(Pragma, false);

			Scanner.EmplaceToken(FHlslToken(EHlslToken::Pragma, Pragma), Tokenizer);
		}
		else if (Tokenizer.MatchString(MATCH_TARGET(TEXT("#if 0"))))
		{
			if (Tokenizer.Peek() == ' ' || Tokenizer.Peek() == '\n')
			{
				Tokenizer.SkipToNextLine();
				while (Tokenizer.HasCharsAvailable() && Tokenizer.Peek() != '#')
				{
					Tokenizer.SkipToNextLine();
				}

				if (Tokenizer.MatchString(MATCH_TARGET(TEXT("#endif"))))
				{
					// Nothing here, skip to next line will happen later
				}
				else
				{
					CompilerMessages.SourceWarning(*FString::Printf(TEXT("Expected #endif preprocessor directive; HlslParser requires preprocessed input!")));
				}
			}
			else
			{
				TStringBuilder<128> Directive;
				Directive += TEXT("#if 0");
				Tokenizer.ReadToEndOfLine(Directive);
				CompilerMessages.SourceWarning(*FString::Printf(TEXT("Unhandled preprocessor directive (%.500s); HlslParser requires preprocessed input!"), *Directive));
			}
		}
		else
		{
			TStringBuilder<128> Directive;
			Tokenizer.ReadToEndOfLine(Directive);
			CompilerMessages.SourceWarning(*FString::Printf(TEXT("Unhandled token (%.500s); HlslParser requires preprocessed input!"), *Directive));
		}

		Tokenizer.SkipToNextLine();
	}
}
