// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HlslLexer.h - Interface for scanning & tokenizing hlsl.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Containers/IndirectArray.h"
#include "HlslUtils.h"

class Error;

namespace CrossCompiler
{
	enum class EHlslToken
	{
		// Control
		Invalid,
		Pragma,

		// Math
		Plus,
		PlusEqual,
		Minus,
		MinusEqual,
		Times,
		TimesEqual,
		Div,
		DivEqual,
		Mod,
		ModEqual,
		LeftParenthesis,
		RightParenthesis,

		// Logical
		EqualEqual,
		NotEqual,
		Lower,
		LowerEqual,
		Greater,
		GreaterEqual,
		AndAnd,
		OrOr,

		// Bit
		LowerLower,
		LowerLowerEqual,
		GreaterGreater,
		GreaterGreaterEqual,
		And,
		AndEqual,
		Or,
		OrEqual,
		Xor,
		XorEqual,
		Not,
		Neg,

		// Statements
		Equal,
		LeftBrace,
		RightBrace,
		Semicolon,
		If,
		Else,
		For,
		While,
		Do,
		Return,
		Switch,
		Case,
		Break,
		Default,
		Continue,
		Goto,

		// Unary
		PlusPlus,
		MinusMinus,

		// Types
		Void,
		Const,
		Precise,

		Bool,
		Bool1,
		Bool2,
		Bool3,
		Bool4,
		Bool1x1,
		Bool2x1,
		Bool3x1,
		Bool4x1,
		Bool1x2,
		Bool2x2,
		Bool3x2,
		Bool4x2,
		Bool1x3,
		Bool2x3,
		Bool3x3,
		Bool4x3,
		Bool1x4,
		Bool2x4,
		Bool3x4,
		Bool4x4,

		Int,
		Int1,
		Int2,
		Int3,
		Int4,
		Int1x1,
		Int2x1,
		Int3x1,
		Int4x1,
		Int1x2,
		Int2x2,
		Int3x2,
		Int4x2,
		Int1x3,
		Int2x3,
		Int3x3,
		Int4x3,
		Int1x4,
		Int2x4,
		Int3x4,
		Int4x4,

		Uint,
		Uint1,
		Uint2,
		Uint3,
		Uint4,
		Uint1x1,
		Uint2x1,
		Uint3x1,
		Uint4x1,
		Uint1x2,
		Uint2x2,
		Uint3x2,
		Uint4x2,
		Uint1x3,
		Uint2x3,
		Uint3x3,
		Uint4x3,
		Uint1x4,
		Uint2x4,
		Uint3x4,
		Uint4x4,

		Uint64_t,
		Uint64_t1,
		Uint64_t2,
		Uint64_t3,
		Uint64_t4,
		Uint64_t1x1,
		Uint64_t2x1,
		Uint64_t3x1,
		Uint64_t4x1,
		Uint64_t1x2,
		Uint64_t2x2,
		Uint64_t3x2,
		Uint64_t4x2,
		Uint64_t1x3,
		Uint64_t2x3,
		Uint64_t3x3,
		Uint64_t4x3,
		Uint64_t1x4,
		Uint64_t2x4,
		Uint64_t3x4,
		Uint64_t4x4,

		Half,
		Half1,
		Half2,
		Half3,
		Half4,
		Half1x1,
		Half2x1,
		Half3x1,
		Half4x1,
		Half1x2,
		Half2x2,
		Half3x2,
		Half4x2,
		Half1x3,
		Half2x3,
		Half3x3,
		Half4x3,
		Half1x4,
		Half2x4,
		Half3x4,
		Half4x4,

		Min16Float,
		Min16Float1,
		Min16Float2,
		Min16Float3,
		Min16Float4,
		Min16Float1x1,
		Min16Float2x1,
		Min16Float3x1,
		Min16Float4x1,
		Min16Float1x2,
		Min16Float2x2,
		Min16Float3x2,
		Min16Float4x2,
		Min16Float1x3,
		Min16Float2x3,
		Min16Float3x3,
		Min16Float4x3,
		Min16Float1x4,
		Min16Float2x4,
		Min16Float3x4,
		Min16Float4x4,

		Float,
		Float1,
		Float2,
		Float3,
		Float4,
		Float1x1,
		Float2x1,
		Float3x1,
		Float4x1,
		Float1x2,
		Float2x2,
		Float3x2,
		Float4x2,
		Float1x3,
		Float2x3,
		Float3x3,
		Float4x3,
		Float1x4,
		Float2x4,
		Float3x4,
		Float4x4,

		Texture, // texture (SM3)
		Texture1D,
		Texture1DArray,
		Texture2D,
		Texture2DArray,
		Texture2DMS,
		Texture2DMSArray,
		Texture3D,
		TextureCube,
		TextureCubeArray,

		Sampler, // sampler (SM3)
		Sampler1D, // sampler1D (SM3)
		Sampler2D, // sampler2D (SM3)
		Sampler3D, // sampler3D (SM3)
		SamplerCube, // samplerCUBE (SM3)
		SamplerState,
		SamplerComparisonState,

		Buffer,
		AppendStructuredBuffer,
		ByteAddressBuffer,
		ConsumeStructuredBuffer,
		RWBuffer,
		RWByteAddressBuffer,
		RWStructuredBuffer,
		RWTexture1D,
		RWTexture1DArray,
		RWTexture2D,
		RWTexture2DArray,
		RasterizerOrderedTexture2D,
		RWTexture3D,
		StructuredBuffer,
		ConstantBuffer,
		RaytracingAccelerationStructure,

		// Modifiers
		In,
		Out,
		InOut,
		Static,
		Uniform,

		// Misc
		LeftSquareBracket,
		RightSquareBracket,
		Question,
		Colon,
		ColonColon,
		Comma,
		Dot,
		Struct,
		CBuffer,
		GroupShared,
		RowMajor,
		Register,
		Inline,
		Typedef,
		PackOffset,
		Namespace,

		Identifier,
		Operator, // HLSL2021
		Literal,
		BoolConstant,
		StringConstant,	// C-style "string"
		StaticAssert, // _Static_assert (HLSL), static_assert (C++11)
		C99Pragma, // _Pragma (C99/C++11)
	};

	enum class ELiteralType
	{
		Unknown = -1,

		Float,			// All variations
		FloatSuffix,	// All variations, ends with 'f'
		Integer,		// Just digits
		IntegerSuffix,	// has a 'u|U|l|L' suffix
		Hex,			// has '0x' or '0X' prefix
		HexSuffix,		// Hex + has a 'u|U|l|L' suffix
		Octal,			// 0[0..7]+
		Bool,			// true or false
	};

	static inline bool IsIntegerType(ELiteralType Type)
	{
		switch (Type)
		{
		case ELiteralType::Integer:
		case ELiteralType::IntegerSuffix:
		case ELiteralType::Hex:
		case ELiteralType::HexSuffix:
		case ELiteralType::Octal:
			return true;
		default:
			break;
		}

		return false;
	}

	struct FHlslToken
	{
		EHlslToken Token;
		ELiteralType LiteralType = ELiteralType::Unknown;
		FString String; // yuriy-todo: allocate this from linear allocator or simply point to the input source code and use FStringView
		FSourceInfo SourceInfo;

		explicit FHlslToken(FStringView Identifier) : Token(EHlslToken::Identifier), String(Identifier) { DebugId(); }
		explicit FHlslToken(EHlslToken InToken, FStringView Identifier) : Token(InToken), String(Identifier) { DebugId(); }
		explicit FHlslToken(FStringView InLiteral, ELiteralType InLiteralType) :
			Token(EHlslToken::Literal),
			LiteralType(InLiteralType),
			String(InLiteral)
			{ DebugId(); }

		void DebugId()
		{
#if UE_BUILD_DEBUG
			static int32 DebugID = 0;
			//ensure(DebugID != 4787);
			++DebugID;
#endif
		}
	};

	class FHlslScanner
	{
	public:
		FHlslScanner(FCompilerMessages& InCompilerMessages);
		virtual ~FHlslScanner();

		// Processing
		bool Lex(const FString& String, const FString& Filename);
		void Dump();

		// Iterating after Processing
		uint32 GetCurrentTokenIndex() const
		{
			return CurrentToken;
		}

		void SetCurrentTokenIndex(uint32 NewToken);

		bool HasMoreTokens() const;
		bool MatchToken(EHlslToken InToken);
		bool MatchIntegerLiteral();
		const FHlslToken* PeekToken(uint32 LookAhead = 0) const;
		const FHlslToken* GetCurrentToken() const;
		void Advance() {++CurrentToken;}

		void SourceError(const FString& Error);

	private:
		FCompilerMessages& CompilerMessages;

		TArray<FHlslToken> Tokens;
		void EmplaceToken(FHlslToken&& Token, const struct FTokenizer& Tokenizer);

		uint32 CurrentToken;

		// Tokens point their source filenames here
		TIndirectArray<FString> SourceFilenames;

		void Clear(const FString& Filename);

		friend struct FTokenizer;
	};
}
