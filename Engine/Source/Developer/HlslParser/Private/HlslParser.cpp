// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HlslParser.cpp - Implementation for parsing hlsl.
=============================================================================*/

#include "HlslParser.h"
#include "HlslParserInternal.h"
#include "HlslAST.h"
#include "HlslLexer.h"
#include "HlslExpressionParser.inl"
#include "Misc/Optional.h"

namespace CrossCompiler
{
	EParseResult ParseExpressionStatement(class FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutStatement);
	EParseResult ParseStructDeclaration(FHlslParser& Parser, FSymbolScope* SymbolScope, FLinearAllocator* Allocator, AST::FTypeSpecifier** OutTypeSpecifier);
	EParseResult TryParseAttribute(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FAttribute** OutAttribute);
	EParseResult ParseFunctionDeclaration(FHlslParser& Parser, FLinearAllocator* Allocator, TLinearArray<AST::FAttribute*>& Attributes, AST::FNode** OutFunction);
	EParseResult TryTranslationUnit(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutNode);

	typedef EParseResult(*TTryRule)(class FHlslParser& Scanner, FLinearAllocator* Allocator, AST::FNode** OutStatement);
	struct FRulePair
	{
		EHlslToken Token;
		TTryRule TryRule;
		bool bSupportsAttributes;

		FRulePair(EHlslToken InToken, TTryRule InTryRule, bool bInSupportsAttributes = false) : Token(InToken), TryRule(InTryRule), bSupportsAttributes(bInSupportsAttributes) {}
	};
	typedef TArray<FRulePair> TRulesArray;

	class FHlslParser
	{
	public:
		FHlslParser(FLinearAllocator* InAllocator, FCompilerMessages& InCompilerMessages, TConstArrayView<UE::HlslParser::FScopedDeclarations> InScopedDeclarations);
		FHlslScanner Scanner;
		FCompilerMessages& CompilerMessages;
		FSymbolScope GlobalScope;
		FSymbolScope Namespaces;
		FSymbolScope* CurrentScope;
		FLinearAllocator* Allocator;
	};

	TRulesArray RulesStatements;

	EParseResult TryStatementRules(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutNode)
	{
		for (const auto& Rule : RulesStatements)
		{
			auto CurrentTokenIndex = Parser.Scanner.GetCurrentTokenIndex();
			TLinearArray<AST::FAttribute*> Attributes(Allocator);
			if (Rule.bSupportsAttributes)
			{
				while (Parser.Scanner.HasMoreTokens())
				{
					const auto* Peek = Parser.Scanner.GetCurrentToken();
					if (Peek->Token == EHlslToken::LeftSquareBracket)
					{
						AST::FAttribute* Attribute = nullptr;
						auto Result = TryParseAttribute(Parser, Allocator, &Attribute);
						if (Result == EParseResult::Matched)
						{
							Attributes.Add(Attribute);
							continue;
						}
						else if (Result == EParseResult::Error)
						{
							return ParseResultError();
						}
					}

					break;
				}
			}

			if (Parser.Scanner.MatchToken(Rule.Token) || Rule.Token == EHlslToken::Invalid)
			{
				AST::FNode* Node = nullptr;
				EParseResult Result = (*Rule.TryRule)(Parser, Allocator, &Node);
				if (Result == EParseResult::Error)
				{
					return ParseResultError();
				}
				else if (Result == EParseResult::Matched)
				{
					if (Attributes.Num() > 0)
					{
						Swap(Node->Attributes, Attributes);
					}
					*OutNode = Node;
					return EParseResult::Matched;
				}
			}

			Parser.Scanner.SetCurrentTokenIndex(CurrentTokenIndex);
		}

		return EParseResult::NotMatched;
	}

	bool MatchPragma(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutNode)
	{
		const auto* Peek = Parser.Scanner.GetCurrentToken();
		if (Parser.Scanner.MatchToken(EHlslToken::Pragma))
		{
			auto* Pragma = new(Allocator)AST::FPragma(Allocator, *Peek->String, Peek->SourceInfo);
			*OutNode = Pragma;
			return true;
		}

		return false;
	}

	EParseResult ParseDeclarationArrayBracketsAndIndex(FHlslScanner& Scanner, FSymbolScope* SymbolScope, bool bNeedsDimension, FLinearAllocator* Allocator, AST::FExpression** OutExpression)
	{
		if (Scanner.MatchToken(EHlslToken::LeftSquareBracket))
		{
			auto ExpressionResult = ParseExpression(Scanner, SymbolScope, 0, Allocator, OutExpression);
			if (ExpressionResult == EParseResult::Error)
			{
				Scanner.SourceError(TEXT("Expected expression!"));
				return ParseResultError();
			}

			if (!Scanner.MatchToken(EHlslToken::RightSquareBracket))
			{
				Scanner.SourceError(TEXT("Expected ']'!"));
				return ParseResultError();
			}

			if (ExpressionResult == EParseResult::NotMatched)
			{
				if (bNeedsDimension)
				{
					Scanner.SourceError(TEXT("Expected array dimension!"));
					return ParseResultError();
				}
			}

			return EParseResult::Matched;
		}

		return EParseResult::NotMatched;
	}

	EParseResult ParseDeclarationMultiArrayBracketsAndIndex(FHlslScanner& Scanner, FSymbolScope* SymbolScope, bool bNeedsDimension, FLinearAllocator* Allocator, AST::FDeclaration* Declaration)
	{
		bool bFoundOne = false;
		do
		{
			AST::FExpression* Dimension = nullptr;
			auto Result = ParseDeclarationArrayBracketsAndIndex(Scanner, SymbolScope, bNeedsDimension, Allocator, &Dimension);
			if (Result == EParseResult::Error)
			{
				return ParseResultError();
			}
			else if (Result == EParseResult::NotMatched)
			{
				break;
			}

			Declaration->ArraySize.Add(Dimension);

			bFoundOne = true;
		}
		while (Scanner.HasMoreTokens());

		if (bFoundOne)
		{
			Declaration->bIsArray = true;
			return EParseResult::Matched;
		}

		return EParseResult::NotMatched;
	}

	// Multi declaration parser flags
	enum EDeclarationFlags
	{
		EDF_CONST_ROW_MAJOR				= (1 << 0),
		EDF_STATIC						= (1 << 1),
		EDF_UNIFORM						= (1 << 2),
		EDF_TEXTURE_SAMPLER_OR_BUFFER	= (1 << 3),
		EDF_INITIALIZER					= (1 << 4),
		EDF_INITIALIZER_LIST			= (1 << 5) | EDF_INITIALIZER,
		EDF_SEMANTIC					= (1 << 6),
		EDF_SEMICOLON					= (1 << 7),
		EDF_IN_OUT						= (1 << 8),
		EDF_MULTIPLE					= (1 << 9),
		EDF_PRIMITIVE_DATA_TYPE			= (1 << 10),
		EDF_SHARED						= (1 << 11),
		EDF_INTERPOLATION				= (1 << 12),
		EDF_REGISTER					= (1 << 13),
		EDF_PACKOFFSET					= (1 << 14),
		EDF_BITPACK					= (1 << 15),
	};

	EParseResult ParseInitializer(FHlslScanner& Scanner, FSymbolScope* SymbolScope, bool bAllowLists, FLinearAllocator* Allocator, AST::FExpression** OutList)
	{
		if (bAllowLists && Scanner.MatchToken(EHlslToken::LeftBrace))
		{
			EParseResult Result = ParseExpressionList2(Scanner, SymbolScope, Allocator, AST::FExpressionList::EType::Braced, OutList);
			if (Result != EParseResult::Matched)
			{
				Scanner.SourceError(TEXT("Invalid initializer list\n"));
				return ParseResultError();
			}

			if (!Scanner.MatchToken(EHlslToken::RightBrace))
			{
				Scanner.SourceError(TEXT("Expected '}'\n"));
				return ParseResultError();
			}

			return EParseResult::Matched;
		}
		else
		{
			//@todo-LH?
			EParseResult Result = ParseExpression(Scanner, SymbolScope, EEF_ALLOW_ASSIGNMENT, Allocator, OutList);
			if (Result == EParseResult::Error)
			{
				Scanner.SourceError(TEXT("Invalid initializer expression\n"));
			}

			return Result;
		}
	}

	EParseResult ParseColonSpecifier(FHlslScanner& Scanner, FLinearAllocator* Allocator, int32 EDFFlags, AST::FSemanticSpecifier** OutSpecifier)
	{
		const auto* Token = Scanner.GetCurrentToken();

		bool bFound = false;
		AST::FSemanticSpecifier::ESpecType Type = AST::FSemanticSpecifier::ESpecType::Semantic;
		bool bTryRegister = (EDFFlags & EDF_REGISTER) == EDF_REGISTER;
		bool bTryPackOffset = (EDFFlags & EDF_PACKOFFSET) == EDF_PACKOFFSET;
		if (bTryRegister && Scanner.MatchToken(EHlslToken::Register))
		{
			bFound = true;
			Type = AST::FSemanticSpecifier::ESpecType::Register;
		}
		else if (bTryPackOffset && Scanner.MatchToken(EHlslToken::PackOffset))
		{
			bFound = true;
			Type = AST::FSemanticSpecifier::ESpecType::PackOffset;
		}
		else if ((EDFFlags & EDF_SEMANTIC) == EDF_SEMANTIC || ((EDFFlags & EDF_BITPACK) == EDF_BITPACK))
		{
			bool bMatched = false;
			if (Scanner.MatchToken(EHlslToken::Identifier))
			{
				bMatched = true;
			}
			else if (((EDFFlags & EDF_BITPACK) == EDF_BITPACK) && Scanner.MatchToken(EHlslToken::Literal))
			{
				bMatched = true;
			}

			if (!bMatched)
			{
				Scanner.SourceError(TEXT("Expected identifier for semantic!"));
				return ParseResultError();
			}

			*OutSpecifier = new(Allocator) AST::FSemanticSpecifier(Allocator, *Token->String, Token->SourceInfo);
			return EParseResult::Matched;
		}

		if (bFound)
		{
			if (!Scanner.MatchToken(EHlslToken::LeftParenthesis))
			{
				Scanner.SourceError(TEXT("'(' expected"));
				return ParseResultError();
			}

			auto* Specifier = new(Allocator) AST::FSemanticSpecifier(Allocator, Type, Token->SourceInfo);

			// Parse argument expressions of 'register'-specifier
			do
			{
				AST::FExpression* Argument = nullptr;
				if (ParseExpression(Scanner, nullptr, 0, Allocator, &Argument) != EParseResult::Matched)
				{
					Scanner.SourceError(TEXT("Expression expected"));
					return ParseResultError();
				}
				Specifier->Arguments.Add(Argument);
			}
			while (Scanner.MatchToken(EHlslToken::Comma));

			if (!Scanner.MatchToken(EHlslToken::RightParenthesis))
			{
				Scanner.SourceError(TEXT("')' expected"));
				return ParseResultError();
			}

			*OutSpecifier = Specifier;
			return EParseResult::Matched;
		}

		return EParseResult::NotMatched;
	}

	EParseResult ParseTextureOrBufferSimpleDeclaration(FHlslScanner& Scanner, FSymbolScope* SymbolScope, bool bMultiple, bool bInitializer, bool bRegister, FLinearAllocator* Allocator, AST::FDeclaratorList** OutDeclaratorList)
	{
		const uint32 OriginalToken = Scanner.GetCurrentTokenIndex();
		const FHlslToken* Token = Scanner.GetCurrentToken();
		AST::FFullySpecifiedType* FullType = (*OutDeclaratorList)->Type;
		if (ParseFullType(Scanner, ETF_SAMPLER_TEXTURE_BUFFER, ETF_BUILTIN_NUMERIC | ETF_USER_TYPES | ETF_UNORM, SymbolScope, Allocator, &FullType) == EParseResult::Matched)
		{
			do
			{
				// Handle 'Sampler2D Sampler'
				AST::FTypeSpecifier* DummyTypeSpecifier = nullptr;
				const auto* IdentifierToken = Scanner.GetCurrentToken();
				AST::FDeclaration* Declaration = nullptr;
				if (ParseGeneralType(Scanner, ETF_SAMPLER_TEXTURE_BUFFER, nullptr, Allocator, &DummyTypeSpecifier) == EParseResult::Matched)
				{
					Declaration = new(Allocator) AST::FDeclaration(Allocator, DummyTypeSpecifier->SourceInfo);
					Declaration->Identifier = AST::FIdentifier::New(Allocator, DummyTypeSpecifier->TypeName);
				}
				else if (Scanner.MatchToken(EHlslToken::Identifier))
				{
					Declaration = new(Allocator) AST::FDeclaration(Allocator, IdentifierToken->SourceInfo);
					Declaration->Identifier = AST::FIdentifier::New(Allocator, IdentifierToken->String);
				}
				else
				{
					Scanner.SourceError(TEXT("Expected Identifier!"));
					return ParseResultError();
				}

				if (ParseDeclarationMultiArrayBracketsAndIndex(Scanner, SymbolScope, true, Allocator, Declaration) == EParseResult::Error)
				{
					return ParseResultError();
				}

				// Parse optional 'register'-specifier
				if (bRegister)
				{
					if (Scanner.MatchToken(EHlslToken::Colon))
					{
						if (ParseColonSpecifier(Scanner, Allocator, EDF_REGISTER, &Declaration->Semantic) != EParseResult::Matched)
						{
							Scanner.SourceError(TEXT("Invalid register specifier"));
							return ParseResultError();
						}
					}
				}

				// Parse optional initializer
				if (bInitializer)
				{
					if (Scanner.MatchToken(EHlslToken::Equal))
					{
						// Setting to true allows declaration of global arrays of textures / buffers
						const bool bAllowInitializerList = true;
						if (ParseInitializer(Scanner, SymbolScope, bAllowInitializerList, Allocator, &Declaration->Initializer) != EParseResult::Matched)
						{
							Scanner.SourceError(TEXT("Invalid initializer\n"));
							return ParseResultError();
						}
					}
				}

				(*OutDeclaratorList)->Declarations.Add(Declaration);
			}
			while (bMultiple && Scanner.MatchToken(EHlslToken::Comma));

			return EParseResult::Matched;
		}

		// Unmatched
		Scanner.SetCurrentTokenIndex(OriginalToken);
		return EParseResult::NotMatched;
	}

	EParseResult ParseDeclarationStorageQualifiers(FHlslScanner& Scanner, int32 TypeFlags, int32 DeclarationFlags, bool& bOutPrimitiveFound, FLinearAllocator* Allocator, AST::FTypeQualifier* Qualifier)
	{
		bOutPrimitiveFound = false;
		int32 StaticFound = 0;
		int32 InterpolationLinearFound = 0;
		int32 InterpolationCentroidFound = 0;
		int32 InterpolationNoInterpolationFound = 0;
		int32 InterpolationNoPerspectiveFound = 0;
		int32 InterpolationSampleFound = 0;
		int32 SharedFound = 0;
		int32 ConstFound = 0;
		int32 RowMajorFound = 0;
		int32 InFound = 0;
		int32 OutFound = 0;
		int32 InOutFound = 0;
		int32 PrimitiveFound = 0;
		int32 UniformFound = 0;

		if (DeclarationFlags & EDF_PRIMITIVE_DATA_TYPE)
		{
			const auto* Token = Scanner.GetCurrentToken();
			if (Token && Token->Token == EHlslToken::Identifier)
			{
				if (Token->String == TEXT("point") ||
					Token->String == TEXT("line") ||
					Token->String == TEXT("triangle") ||
					Token->String == TEXT("Triangle") ||	// PSSL
					Token->String == TEXT("AdjacentLine") ||	// PSSL
					Token->String == TEXT("lineadj") ||
					Token->String == TEXT("AdjacentTriangle") ||	// PSSL
					Token->String == TEXT("triangleadj"))
				{
					Scanner.Advance();
					Qualifier->PrimitiveType = Allocator->Strdup(Token->String);
					++PrimitiveFound;
				}
			}
		}

		while (Scanner.HasMoreTokens())
		{
			bool bFound = false;
			auto* Token = Scanner.GetCurrentToken();
			if ((DeclarationFlags & EDF_STATIC) && Scanner.MatchToken(EHlslToken::Static))
			{
				++StaticFound;
				Qualifier->bIsStatic = true;
				if (StaticFound > 1)
				{
					Scanner.SourceError(TEXT("'static' found more than once!\n"));
					return ParseResultError();
				}
			}
			else if ((DeclarationFlags & EDF_SHARED) && Scanner.MatchToken(EHlslToken::GroupShared))
			{
				++SharedFound;
				Qualifier->bShared = true;
				if (SharedFound > 1)
				{
					Scanner.SourceError(TEXT("'groupshared' found more than once!\n"));
					return ParseResultError();
				}
			}
			else if ((DeclarationFlags & EDF_CONST_ROW_MAJOR) && Scanner.MatchToken(EHlslToken::Const))
			{
				++ConstFound;
				Qualifier->bConstant = true;
				if (ConstFound > 1)
				{
					Scanner.SourceError(TEXT("'const' found more than once!\n"));
					return ParseResultError();
				}
			}
			else if ((DeclarationFlags & EDF_CONST_ROW_MAJOR) && Scanner.MatchToken(EHlslToken::RowMajor))
			{
				++RowMajorFound;
				Qualifier->bRowMajor = true;
				if (RowMajorFound > 1)
				{
					Scanner.SourceError(TEXT("'row_major' found more than once!\n"));
					return ParseResultError();
				}
			}
			else if ((DeclarationFlags & EDF_IN_OUT) && Scanner.MatchToken(EHlslToken::In))
			{
				++InFound;
				Qualifier->bIn = true;
				if (InFound > 1)
				{
					Scanner.SourceError(TEXT("'in' found more than once!\n"));
					return ParseResultError();
				}
				else if (InOutFound > 0)
				{
					Scanner.SourceError(TEXT("'in' can't be used with 'inout'!\n"));
					return ParseResultError();
				}
			}
			else if ((DeclarationFlags & EDF_IN_OUT) && Scanner.MatchToken(EHlslToken::Out))
			{
				++OutFound;
				Qualifier->bOut = true;
				if (OutFound > 1)
				{
					Scanner.SourceError(TEXT("'out' found more than once!\n"));
					return ParseResultError();
				}
				else if (InOutFound > 0)
				{
					Scanner.SourceError(TEXT("'out' can't be used with 'inout'!\n"));
					return ParseResultError();
				}
			}
			else if ((DeclarationFlags & EDF_IN_OUT) && Scanner.MatchToken(EHlslToken::InOut))
			{
				++InOutFound;
				Qualifier->bIn = true;
				Qualifier->bOut = true;
				if (InOutFound > 1)
				{
					Scanner.SourceError(TEXT("'inout' found more than once!\n"));
					return ParseResultError();
				}
				else if (InFound > 0 || OutFound > 0)
				{
					Scanner.SourceError(TEXT("'inout' can't be used with 'in' or 'out'!\n"));
					return ParseResultError();
				}
			}
			else if ((DeclarationFlags & EDF_UNIFORM) && Scanner.MatchToken(EHlslToken::Uniform))
			{
				++UniformFound;
				Qualifier->bUniform = true;
				if (UniformFound > 1)
				{
					Scanner.SourceError(TEXT("'uniform' found more than once!\n"));
					return ParseResultError();
				}
			}
			else if ((DeclarationFlags & EDF_INTERPOLATION) && Token->Token == EHlslToken::Identifier)
			{
				if (Token->String == TEXT("linear"))
				{
					Scanner.Advance();
					++InterpolationLinearFound;
					Qualifier->bLinear = true;
					if (InterpolationLinearFound > 1)
					{
						Scanner.SourceError(TEXT("'linear' found more than once!\n"));
						return ParseResultError();
					}
				}
				else if (Token->String == TEXT("centroid"))
				{
					Scanner.Advance();
					++InterpolationCentroidFound;
					Qualifier->bCentroid = true;
					if (InterpolationCentroidFound > 1)
					{
						Scanner.SourceError(TEXT("'centroid' found more than once!\n"));
						return ParseResultError();
					}
				}
				else if (Token->String == TEXT("nointerpolation") || Token->String == TEXT("nointerp"))
				{
					Scanner.Advance();
					++InterpolationNoInterpolationFound;
					Qualifier->NoInterpolatorType = Allocator->Strdup(Token->String);
					if (InterpolationNoInterpolationFound > 1)
					{
						Scanner.SourceError(TEXT("'nointerpolation' found more than once!\n"));
						return ParseResultError();
					}
				}
				else if (Token->String == TEXT("noperspective") || Token->String == TEXT("nopersp"))	
				{
					Scanner.Advance();
					++InterpolationNoPerspectiveFound;
					Qualifier->NoPerspectiveType = Allocator->Strdup(Token->String);
					if (InterpolationNoPerspectiveFound > 1)
					{
						Scanner.SourceError(TEXT("'noperspective' found more than once!\n"));
						return ParseResultError();
					}
				}
				else if (Token->String == TEXT("sample"))
				{
					Scanner.Advance();
					++InterpolationSampleFound;
					Qualifier->bSample = true;
					if (InterpolationSampleFound > 1)
					{
						Scanner.SourceError(TEXT("'sample' found more than once!\n"));
						return ParseResultError();
					}
				}
				else
				{
					break;
				}
			}
			else
			{
				break;
			}
		}

		int32 InterpolationFound = InterpolationLinearFound + InterpolationCentroidFound + InterpolationNoInterpolationFound + InterpolationNoPerspectiveFound + InterpolationSampleFound;
		if (InterpolationFound)
		{
			if (InterpolationLinearFound && InterpolationNoInterpolationFound)
			{
				Scanner.SourceError(TEXT("Can't have both 'linear' and 'nointerpolation'!\n"));
				return ParseResultError();
			}

			if (InterpolationCentroidFound && !(InterpolationLinearFound || InterpolationNoPerspectiveFound))
			{
				Scanner.SourceError(TEXT("'centroid' must be used with either 'linear' or 'noperspective'!\n"));
				return ParseResultError();
			}
		}

		if (UniformFound && (OutFound || InOutFound || PrimitiveFound || SharedFound || InterpolationFound))
		{
			Scanner.SourceError(TEXT("'uniform' can not be used with other storage qualifiers (inout, out, nointerpolation, etc)!\n"));
			return ParseResultError();
		}

		bOutPrimitiveFound = (PrimitiveFound > 0);

		return (ConstFound + RowMajorFound + InFound + OutFound + InOutFound + StaticFound + SharedFound + PrimitiveFound + InterpolationFound + UniformFound)
			? EParseResult::Matched
			: EParseResult::NotMatched;
	}

	EParseResult ParseGeneralDeclarationNoSemicolon(FHlslParser& Parser, FSymbolScope* SymbolScope, int32 TypeFlags, int32 DeclarationFlags, FLinearAllocator* Allocator, AST::FDeclaratorList** OutDeclaratorList)
	{
		auto OriginalToken = Parser.Scanner.GetCurrentTokenIndex();
		bool bPrimitiveFound = false;
		auto* FullType = new(Allocator) AST::FFullySpecifiedType(Allocator, Parser.Scanner.GetCurrentToken()->SourceInfo);
		auto ParseResult = ParseDeclarationStorageQualifiers(Parser.Scanner, TypeFlags, DeclarationFlags, bPrimitiveFound, Allocator, &FullType->Qualifier);
		if (ParseResult == EParseResult::Error)
		{
			return ParseResultError();
		}
		bool bCanBeUnmatched = (ParseResult == EParseResult::NotMatched);

		auto* DeclaratorList = new(Allocator) AST::FDeclaratorList(Allocator, FullType->SourceInfo);
		DeclaratorList->Type = FullType;

		if (!bPrimitiveFound && (DeclarationFlags & EDF_PRIMITIVE_DATA_TYPE))
		{
			const auto* StreamToken = Parser.Scanner.GetCurrentToken();
			if (StreamToken)
			{
				if (StreamToken->Token == EHlslToken::Identifier)
				{
					if (StreamToken->String == TEXT("PointStream") ||
						StreamToken->String == TEXT("PointBuffer") ||	// PSSL
						StreamToken->String == TEXT("LineStream") ||
						StreamToken->String == TEXT("LineBuffer") ||	// PSSL
						StreamToken->String == TEXT("TriangleStream") ||
						StreamToken->String == TEXT("TriangleBuffer") ||	// PSSL
						StreamToken->String == TEXT("InputPatch") ||
						StreamToken->String == TEXT("OutputPatch"))
					{
						Parser.Scanner.Advance();
						bCanBeUnmatched = false;

						if (!Parser.Scanner.MatchToken(EHlslToken::Lower))
						{
							Parser.Scanner.SourceError(TEXT("Expected '<'!"));
							return ParseResultError();
						}

						AST::FTypeSpecifier* TypeSpecifier = nullptr;
						if (ParseGeneralType(Parser.Scanner, ETF_BUILTIN_NUMERIC | ETF_USER_TYPES, SymbolScope, Allocator, &TypeSpecifier) != EParseResult::Matched)
						{
							Parser.Scanner.SourceError(TEXT("Expected type!"));
							return ParseResultError();
						}

						if (StreamToken->String == TEXT("InputPatch") || StreamToken->String == TEXT("OutputPatch"))
						{
							if (!Parser.Scanner.MatchToken(EHlslToken::Comma))
							{
								Parser.Scanner.SourceError(TEXT("Expected ','!"));
								return ParseResultError();
							}

							//@todo-rco: Save this value!
							auto* Elements = Parser.Scanner.GetCurrentToken();
							if (!Parser.Scanner.MatchIntegerLiteral())
							{
								Parser.Scanner.SourceError(TEXT("Expected number!"));
								return ParseResultError();
							}

							TypeSpecifier->TextureMSNumSamples = FCString::Atoi(*Elements->String);
						}

						if (!Parser.Scanner.MatchToken(EHlslToken::Greater))
						{
							Parser.Scanner.SourceError(TEXT("Expected '>'!"));
							return ParseResultError();
						}

						auto* IdentifierToken = Parser.Scanner.GetCurrentToken();
						if (!Parser.Scanner.MatchToken(EHlslToken::Identifier))
						{
							Parser.Scanner.SourceError(TEXT("Expected identifier!"));
							return ParseResultError();
						}

						TypeSpecifier->InnerType = TypeSpecifier->TypeName;
						TypeSpecifier->TypeName = Allocator->Strdup(StreamToken->String);
						FullType->Specifier = TypeSpecifier;

						auto* Declaration = new(Allocator)AST::FDeclaration(Allocator, IdentifierToken->SourceInfo);
						Declaration->Identifier = FindOrAddIdentifier(Allocator, SymbolScope, IdentifierToken->String);

						DeclaratorList->Declarations.Add(Declaration);
						*OutDeclaratorList = DeclaratorList;

						// Adding variable declarations to symbols defined within specific namespace
						if (SymbolScope && SymbolScope->Name != nullptr)
						{
							SymbolScope->Symbols.Add(IdentifierToken->String);
						}

						return EParseResult::Matched;
					}
				}
			}
		}

		if (DeclarationFlags & EDF_TEXTURE_SAMPLER_OR_BUFFER)
		{
			bool bMultiple = ((DeclarationFlags & EDF_MULTIPLE) == EDF_MULTIPLE);
			bool bInitializer = ((DeclarationFlags & EDF_INITIALIZER) == EDF_INITIALIZER);
			bool bRegister = ((DeclarationFlags & EDF_REGISTER) == EDF_REGISTER);

			auto Result = ParseTextureOrBufferSimpleDeclaration(Parser.Scanner, SymbolScope, bMultiple, bInitializer, bRegister, Allocator, &DeclaratorList);
			if (Result == EParseResult::Matched)
			{
				*OutDeclaratorList = DeclaratorList;
				return EParseResult::Matched;
			}
			else if (Result == EParseResult::Error)
			{
				return ParseResultError();
			}
		}

		const bool bAllowInitializerList = (DeclarationFlags & EDF_INITIALIZER_LIST) == EDF_INITIALIZER_LIST;

		if (Parser.Scanner.MatchToken(EHlslToken::Struct))
		{
			auto Result = ParseStructDeclaration(Parser, SymbolScope, Allocator, &FullType->Specifier);
			if (Result != EParseResult::Matched)
			{
				return ParseResultError();
			}

			do
			{
				auto* IdentifierToken = Parser.Scanner.GetCurrentToken();
				if (Parser.Scanner.MatchToken(EHlslToken::Identifier))
				{
					//... Instance

					auto* Declaration = new(Allocator) AST::FDeclaration(Allocator, IdentifierToken->SourceInfo);
					Declaration->Identifier = FindOrAddIdentifier(Allocator, SymbolScope, IdentifierToken->String);

					if (ParseDeclarationMultiArrayBracketsAndIndex(Parser.Scanner, SymbolScope, false, Allocator, Declaration) == EParseResult::Error)
					{
						return ParseResultError();
					}

					if (DeclarationFlags & EDF_INITIALIZER)
					{
						if (Parser.Scanner.MatchToken(EHlslToken::Equal))
						{
							if (ParseInitializer(Parser.Scanner, SymbolScope, bAllowInitializerList, Allocator, &Declaration->Initializer) != EParseResult::Matched)
							{
								Parser.Scanner.SourceError(TEXT("Invalid initializer\n"));
								return ParseResultError();
							}
						}
					}
				
					DeclaratorList->Declarations.Add(Declaration);
				}
			}
			while ((DeclarationFlags & EDF_MULTIPLE) == EDF_MULTIPLE && Parser.Scanner.MatchToken(EHlslToken::Comma));
			*OutDeclaratorList = DeclaratorList;
		}
		else
		{
			auto Result = ParseGeneralType(Parser.Scanner, ETF_BUILTIN_NUMERIC | ETF_USER_TYPES | (TypeFlags & ETF_ERROR_IF_NOT_USER_TYPE), SymbolScope, Allocator, &FullType->Specifier);
			if (Result == EParseResult::Matched)
			{
				bool bMatched = false;
				do
				{
					auto* IdentifierToken = Parser.Scanner.GetCurrentToken();
					if (Parser.Scanner.MatchToken(EHlslToken::Texture) || Parser.Scanner.MatchToken(EHlslToken::Sampler) || Parser.Scanner.MatchToken(EHlslToken::Buffer))
					{
						// Continue, handles the case of 'float3 Texture'...
					}
					else if (!Parser.Scanner.MatchToken(EHlslToken::Identifier))
					{
						Parser.Scanner.SetCurrentTokenIndex(OriginalToken);
						return EParseResult::NotMatched;
					}

					auto* Declaration = new(Allocator) AST::FDeclaration(Allocator, IdentifierToken->SourceInfo);
					Declaration->Identifier = FindOrAddIdentifier(Allocator, SymbolScope, IdentifierToken->String);
					//AddVar

					if (ParseDeclarationMultiArrayBracketsAndIndex(Parser.Scanner, SymbolScope, false, Allocator, Declaration) == EParseResult::Error)
					{
						return ParseResultError();
					}

					bool bSemanticFound = false;
					if ((DeclarationFlags & (EDF_REGISTER | EDF_PACKOFFSET | EDF_SEMANTIC | EDF_BITPACK)) != 0)
					{
						if (Parser.Scanner.MatchToken(EHlslToken::Colon))
						{
							if (ParseColonSpecifier(Parser.Scanner, Allocator, (DeclarationFlags & (EDF_SEMANTIC | EDF_REGISTER | EDF_PACKOFFSET | EDF_BITPACK)), &Declaration->Semantic) != EParseResult::Matched)
							{
								Parser.Scanner.SourceError(TEXT("Expected semantic, register or packoffset!\n"));
								return ParseResultError();
							}
							bSemanticFound = true;
						}
					}

					if ((DeclarationFlags & EDF_INITIALIZER) && !bSemanticFound)
					{
						if (Parser.Scanner.MatchToken(EHlslToken::Equal))
						{
							if (ParseInitializer(Parser.Scanner, SymbolScope, bAllowInitializerList, Allocator, &Declaration->Initializer) != EParseResult::Matched)
							{
								Parser.Scanner.SourceError(TEXT("Invalid initializer\n"));
								return ParseResultError();
							}
						}
					}

					DeclaratorList->Declarations.Add(Declaration);
				}
				while ((DeclarationFlags & EDF_MULTIPLE) == EDF_MULTIPLE && Parser.Scanner.MatchToken(EHlslToken::Comma));

				*OutDeclaratorList = DeclaratorList;
			}
			else if (Result == EParseResult::Error)
			{
				return ParseResultError();
			}
			else if (bCanBeUnmatched && Result == EParseResult::NotMatched)
			{
				Parser.Scanner.SetCurrentTokenIndex(OriginalToken);
				return EParseResult::NotMatched;
			}
		}

		return EParseResult::Matched;
	}

	EParseResult ParseGeneralDeclaration(FHlslParser& Parser, FSymbolScope* SymbolScope, FLinearAllocator* Allocator, AST::FDeclaratorList** OutDeclaration, int32 TypeFlags, int32 DeclarationFlags)
	{
		auto Result = ParseGeneralDeclarationNoSemicolon(Parser, SymbolScope, TypeFlags, DeclarationFlags, Allocator, OutDeclaration);
		if (Result == EParseResult::NotMatched || Result == EParseResult::Error)
		{
			return Result;
		}

		if (DeclarationFlags & EDF_SEMICOLON)
		{
			if (!Parser.Scanner.MatchToken(EHlslToken::Semicolon))
			{
				Parser.Scanner.SourceError(TEXT("';' expected!\n"));
				return ParseResultError();
			}
		}

		return EParseResult::Matched;
	}

	EParseResult ParseCBuffer(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutDeclaration)
	{
		const auto* Token = Parser.Scanner.GetCurrentToken();
		if (!Token)
		{
			Parser.Scanner.SourceError(TEXT("Expected '{'!"));
			return ParseResultError();
		}

		auto* CBuffer = new(Allocator) AST::FCBufferDeclaration(Allocator, Token->SourceInfo);
		if (Parser.Scanner.MatchToken(EHlslToken::Identifier))
		{
			CBuffer->Name = Allocator->Strdup(Token->String);
		}

		bool bFoundRightBrace = false;
		if (Parser.Scanner.MatchToken(EHlslToken::LeftBrace))
		{
			while (Parser.Scanner.HasMoreTokens())
			{
				if (Parser.Scanner.MatchToken(EHlslToken::RightBrace))
				{
					if (Parser.Scanner.MatchToken(EHlslToken::Semicolon))
					{
						// Optional???
					}

					*OutDeclaration = CBuffer;
					return EParseResult::Matched;
				}

				AST::FDeclaratorList* Declaration = nullptr;
				auto Result = ParseGeneralDeclaration(Parser, Parser.CurrentScope, Allocator, &Declaration, 0, EDF_CONST_ROW_MAJOR | EDF_SEMICOLON | EDF_TEXTURE_SAMPLER_OR_BUFFER | EDF_PACKOFFSET);
				if (Result == EParseResult::Error)
				{
					return ParseResultError();
				}
				else if (Result == EParseResult::NotMatched)
				{
					break;
				}
				CBuffer->Declarations.Add(Declaration);
			}
		}

		Parser.Scanner.SourceError(TEXT("Expected '}'!"));
		return ParseResultError();
	}

	static EParseResult ParseStructBody(FHlslParser& Parser, FSymbolScope* SymbolScope, FLinearAllocator* Allocator, AST::FStructSpecifier* Struct)
	{
		check(SymbolScope != nullptr);
		check(Allocator != nullptr);
		check(Struct != nullptr);

		bool bFoundRightBrace = false;
		while (Parser.Scanner.HasMoreTokens())
		{
			if (Parser.Scanner.MatchToken(EHlslToken::RightBrace))
			{
				bFoundRightBrace = true;
				break;
			}

			// Greedily eat a member declaration, and backtrack if it's a member function
			const uint32 OriginalToken = Parser.Scanner.GetCurrentTokenIndex();
			AST::FDeclaratorList* Declaration = nullptr;
			EParseResult Result = ParseGeneralDeclarationNoSemicolon(Parser, SymbolScope, 0, EDF_CONST_ROW_MAJOR | EDF_SEMANTIC | EDF_MULTIPLE | EDF_TEXTURE_SAMPLER_OR_BUFFER | EDF_INTERPOLATION | EDF_BITPACK, Allocator, &Declaration);
			bool bTryMemberFunction = false;
			if (Result == EParseResult::Error)
			{
				return ParseResultError();
			}
			else if (Result == EParseResult::NotMatched)
			{
				bTryMemberFunction = true;
			}
			else
			{
				check(Result == EParseResult::Matched);
				if (Parser.Scanner.MatchToken(EHlslToken::Semicolon))
				{
					// Regular member
					Struct->Members.Add(Declaration);
					continue;
				}
				else
				{
					// Member function, so try again?
					if (Parser.Scanner.MatchToken(EHlslToken::LeftParenthesis))
					{
						Parser.Scanner.SetCurrentTokenIndex(OriginalToken);
						bTryMemberFunction = true;
					}
				}
			}

			if (bTryMemberFunction)
			{
				CrossCompiler::AST::FNode* Function = nullptr;
				TLinearArray<AST::FAttribute*> Attributes(Allocator);
				Result = ParseFunctionDeclaration(Parser, Allocator, Attributes, &Function);
				if (Result == EParseResult::Matched)
				{
					Swap(Function->Attributes, Attributes);
					Struct->Members.Add(Function);
					continue;
				}
				else
				{
					return ParseResultError();
				}
			}
			else
			{
				// Badness. How can did get here?
				check(Result == EParseResult::NotMatched);
				break;
			}
		}

		if (!bFoundRightBrace)
		{
			Parser.Scanner.SourceError(TEXT("Expected '}'!"));
			return ParseResultError();
		}

		return EParseResult::Matched;
	}
		
	EParseResult ParseStructDeclaration(FHlslParser& Parser, FSymbolScope* SymbolScope, FLinearAllocator* Allocator, AST::FTypeSpecifier** OutTypeSpecifier)
	{
		const auto* Name = Parser.Scanner.GetCurrentToken();
		if (!Name)
		{
			return ParseResultError();
		}

		bool bAnonymous = true;
		if (Parser.Scanner.MatchToken(EHlslToken::Identifier))
		{
			bAnonymous = false;
			SymbolScope->Add(Name->String);
		}

		const TCHAR* Parent = nullptr;
		if (Parser.Scanner.MatchToken(EHlslToken::Colon))
		{
			const auto* ParentToken = Parser.Scanner.GetCurrentToken();
			if (!Parser.Scanner.MatchToken(EHlslToken::Identifier))
			{
				Parser.Scanner.SourceError(TEXT("Identifier expected!\n"));
				return ParseResultError();
			}

			Parent = Allocator->Strdup(ParentToken->String);
		}

		const bool bForwardDeclaration = !Parser.Scanner.MatchToken(EHlslToken::LeftBrace);
		if (bForwardDeclaration && bAnonymous)
		{
			Parser.Scanner.SourceError(TEXT("Anonymous struct must have a body definition"));
			return ParseResultError();
		}

		auto* Struct = new(Allocator) AST::FStructSpecifier(Allocator, Name->SourceInfo);
		Struct->ParentName = Allocator->Strdup(Parent);
		//@todo-rco: Differentiate anonymous!
		Struct->Name = bAnonymous ? nullptr : Allocator->Strdup(Name->String);
		Struct->bForwardDeclaration = bForwardDeclaration;

		if (!bForwardDeclaration)
		{
			const EParseResult Result = ParseStructBody(Parser, SymbolScope, Allocator, Struct);
			if (Result != EParseResult::Matched)
			{
				return Result;
			}
		}

		auto* TypeSpecifier = new(Allocator) AST::FTypeSpecifier(Allocator, Struct->SourceInfo);
		TypeSpecifier->Structure = Struct;
		*OutTypeSpecifier = TypeSpecifier;
		return EParseResult::Matched;
	}

	EParseResult ParseFunctionParameterDeclaration(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FFunction* Function)
	{
		bool bStrictCheck = false;

		// To help debug
		AST::FDeclaratorList* PrevDeclaration = nullptr;

		while (Parser.Scanner.HasMoreTokens())
		{
			AST::FDeclaratorList* Declaration = nullptr;
			auto Result = ParseGeneralDeclaration(Parser, Parser.CurrentScope, Allocator, &Declaration, ETF_USER_TYPES | ETF_ERROR_IF_NOT_USER_TYPE, EDF_CONST_ROW_MAJOR | EDF_IN_OUT | EDF_TEXTURE_SAMPLER_OR_BUFFER | EDF_INITIALIZER | EDF_SEMANTIC | EDF_PRIMITIVE_DATA_TYPE | EDF_INTERPOLATION | EDF_UNIFORM);
			if (Result == EParseResult::NotMatched)
			{
				auto* Token = Parser.Scanner.PeekToken();
				if (Token->Token == EHlslToken::RightParenthesis)
				{
					break;
				}

				Parser.Scanner.SourceError(TEXT("Unknown type '") + Token->String + TEXT("'!\n"));
				return ParseResultError();
			}

			if (Result == EParseResult::Error)
			{
				return ParseResultError();
			}
			
			auto* Parameter = AST::FParameterDeclarator::CreateFromDeclaratorList(Declaration, Allocator);
			Function->Parameters.Add(Parameter);
			if (!Parser.Scanner.MatchToken(EHlslToken::Comma))
			{
				break;
			}
			else if (Result == EParseResult::NotMatched) //-V547
			{
				Parser.Scanner.SourceError(TEXT("Internal error on function parameter!\n"));
				return ParseResultError();
			}

			PrevDeclaration = Declaration;
		}

		return EParseResult::Matched;
	}

	static bool IsOverloadableOperatorToken(EHlslToken Token)
	{
		switch (Token)
		{
		// Math
		case EHlslToken::Plus:
		case EHlslToken::PlusEqual:
		case EHlslToken::Minus:
		case EHlslToken::MinusEqual:
		case EHlslToken::Times:
		case EHlslToken::TimesEqual:
		case EHlslToken::Div:
		case EHlslToken::DivEqual:
		case EHlslToken::Mod:
		case EHlslToken::ModEqual:

		// Logical
		case EHlslToken::EqualEqual:
		case EHlslToken::NotEqual:
		case EHlslToken::Lower:
		case EHlslToken::LowerEqual:
		case EHlslToken::Greater:
		case EHlslToken::GreaterEqual:

		// Bit
		case EHlslToken::LowerLower:
		case EHlslToken::LowerLowerEqual:
		case EHlslToken::GreaterGreater:
		case EHlslToken::GreaterGreaterEqual:
		case EHlslToken::And:
		case EHlslToken::AndEqual:
		case EHlslToken::Or:
		case EHlslToken::OrEqual:
		case EHlslToken::Xor:
		case EHlslToken::XorEqual:
		case EHlslToken::Not:
		case EHlslToken::Neg:

		// Statements
		case EHlslToken::Equal:

		// Unary
		case EHlslToken::PlusPlus:
		case EHlslToken::MinusMinus:
			return true;

		default:
			return false;
		}
	}

	static const TCHAR* TryParseOverloadableOperator(FHlslParser& Parser, FLinearAllocator* Allocator)
	{
		const FHlslToken* Token = Parser.Scanner.GetCurrentToken();
		if (!Token)
		{
			return nullptr;
		}
	
		if (Parser.Scanner.MatchToken(EHlslToken::LeftParenthesis))
		{
			// Match invocation operator '()'
			if (!Parser.Scanner.MatchToken(EHlslToken::RightParenthesis))
			{
				Parser.Scanner.SourceError(TEXT("Missing closing ')' token for invocation operator '()'"));
				return nullptr;
			}
			return TEXT("()");
		}
		else if (Parser.Scanner.MatchToken(EHlslToken::LeftSquareBracket))
		{
			// Match subscript operator '[]'
			if (!Parser.Scanner.MatchToken(EHlslToken::RightSquareBracket))
			{
				Parser.Scanner.SourceError(TEXT("Missing closing ']' token for subscript operator '[]'"));
				return nullptr;
			}
			return TEXT("[]");
		}
		else if (IsOverloadableOperatorToken(Token->Token))
		{
			Parser.Scanner.Advance();
			return Allocator->Strdup(Token->String);
		}
		return nullptr;
	}

	EParseResult ParseFunctionReturnType(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FFullySpecifiedType** OutFullySpecifiedType)
	{
		const int32 TypeFlags = ETF_BUILTIN_NUMERIC | ETF_SAMPLER_TEXTURE_BUFFER | ETF_USER_TYPES | ETF_ERROR_IF_NOT_USER_TYPE | ETF_VOID;
		const int32 TemplateTypeFlags = ETF_BUILTIN_NUMERIC | ETF_SAMPLER_TEXTURE_BUFFER | ETF_USER_TYPES | ETF_UNORM;
		return ParseFullType(Parser.Scanner, TypeFlags, TemplateTypeFlags, Parser.CurrentScope, Allocator, OutFullySpecifiedType);
	}

	EParseResult ParseFunctionDeclarator(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FFunction** OutFunction, TOptional<FCreateSymbolScope>* OutScope)
	{
		uint32 OriginalToken = Parser.Scanner.GetCurrentTokenIndex();
		AST::FFullySpecifiedType* ReturnType = nullptr;
		EParseResult Result = ParseFunctionReturnType(Parser, Allocator, &ReturnType);
		if (Result == EParseResult::NotMatched)
		{
			Parser.Scanner.SetCurrentTokenIndex(OriginalToken);
			return EParseResult::NotMatched;
		}
		else if (Result == EParseResult::Error)
		{
			return Result;
		}

		check(Result == EParseResult::Matched);

		const TCHAR* OperatorIdentifier = nullptr;
		const FHlslToken* Identifier = nullptr;

		auto MatchIdentifierOrOperator = [OriginalToken, Allocator](FHlslParser& Parser, const FHlslToken*& Identifier, const TCHAR*& OperatorIdentifier) -> EParseResult
		{
			if (Parser.Scanner.MatchToken(EHlslToken::Operator))
			{
				OperatorIdentifier = TryParseOverloadableOperator(Parser, Allocator);
				if (!OperatorIdentifier)
				{
					Parser.Scanner.SourceError(TEXT("Expected operator token after 'operator' keyword!\n"));
					return ParseResultError();
				}
			}
			else if (!Parser.Scanner.MatchToken(EHlslToken::Identifier))
			{
				// This could be an error... But we should allow testing for a global variable before any rash decisions
				Parser.Scanner.SetCurrentTokenIndex(OriginalToken);
				return EParseResult::NotMatched;
			}
			return EParseResult::Matched;
		};

		Identifier = Parser.Scanner.GetCurrentToken();
		Result = MatchIdentifierOrOperator(Parser, Identifier, OperatorIdentifier);
		if (Result != EParseResult::Matched)
		{
			return Result;
		}

		// Check for scoped function declarations, e.g. "MyStruct::MyFunction() ..."
		const FHlslToken* ScopeIdentifier = nullptr;
		if (OperatorIdentifier == nullptr && Parser.Scanner.MatchToken(EHlslToken::ColonColon))
		{
			ScopeIdentifier = Identifier;
			Result = MatchIdentifierOrOperator(Parser, Identifier, OperatorIdentifier);
			if (Result != EParseResult::Matched)
			{
				return Result;
			}
		}

		if (!Parser.Scanner.MatchToken(EHlslToken::LeftParenthesis))
		{
			// This could be an error... But we should allow testing for a global variable before any rash decisions
			Parser.Scanner.SetCurrentTokenIndex(OriginalToken);
			return EParseResult::NotMatched;
		}

		// At this point, any unknown identifiers could be a type being used without forward declaring, so it's a real error

		auto* Function = new(Allocator) AST::FFunction(Allocator, Identifier->SourceInfo);
		if (OperatorIdentifier)
		{
			Function->Identifier = AST::FIdentifier::New(Allocator, OperatorIdentifier);
			Function->bIsOperator = true;
		}
		else
		{
			Function->Identifier = FindOrAddIdentifier(Allocator, Parser.CurrentScope, Identifier->String);
		}
		if (ScopeIdentifier)
		{
			Function->ScopeIdentifier = AST::FIdentifier::New(Allocator, ScopeIdentifier->String);
		}
		Function->ReturnType = ReturnType;

		if (Parser.Scanner.MatchToken(EHlslToken::Void))
		{
			// Nothing to do here...
		}
		else if (Parser.Scanner.MatchToken(EHlslToken::RightParenthesis))
		{
			goto Done;
		}
		else
		{
			if (OutScope)
			{
				OutScope->Emplace(Allocator, &Parser.CurrentScope, FSymbolScope::EType::Function);
			}

			Result = ParseFunctionParameterDeclaration(Parser, Allocator, Function);
			if (Result == EParseResult::Error)
			{
				return ParseResultError();
			}
		}

		if (!Parser.Scanner.MatchToken(EHlslToken::RightParenthesis))
		{
			Parser.Scanner.SourceError(TEXT("')' expected"));
			return ParseResultError();
		}

Done:
		*OutFunction = Function;

		return EParseResult::Matched;
	}

	EParseResult ParseStatement(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutStatement)
	{
		if (MatchPragma(Parser, Allocator, OutStatement))
		{
			return EParseResult::Matched;
		}

		const auto* Token = Parser.Scanner.PeekToken();
		if (Token && Token->Token == EHlslToken::RightBrace)
		{
			return EParseResult::NotMatched;
		}

		return TryStatementRules(Parser, Allocator, OutStatement);
	}

	EParseResult ParseNamespaceBlock(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutStatement)
	{
		const FHlslToken* Token = Parser.Scanner.PeekToken();
		const FHlslToken* Token1 = Parser.Scanner.PeekToken(1);
		const FHlslToken* Token2 = Parser.Scanner.PeekToken(2);

		if (Token && Token->Token == EHlslToken::Namespace && 
			Token1 && Token1->Token == EHlslToken::Identifier && 
			Token2 && Token2->Token == EHlslToken::LeftBrace)
		{
			FCreateSymbolScope NamespaceSymbolScope(Allocator, &Parser.CurrentScope, FSymbolScope::EType::Namespace);
			Parser.CurrentScope->Name = *(Token1->String);

			check(Parser.Scanner.MatchToken(EHlslToken::Namespace));
			check(Parser.Scanner.MatchToken(EHlslToken::Identifier));
			check(Parser.Scanner.MatchToken(EHlslToken::LeftBrace));

			do
			{
				auto Result = TryTranslationUnit(Parser, Allocator, OutStatement);
				if (Result != EParseResult::Matched)
				{
					return ParseResultError();
				}
			}while(!Parser.Scanner.MatchToken(EHlslToken::RightBrace));

			return EParseResult::Matched;
		}

		return EParseResult::NotMatched;
	}
	
	EParseResult ParseStatementBlock(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutStatement)
	{
		FCreateSymbolScope SymbolScope(Allocator, &Parser.CurrentScope, FSymbolScope::EType::Statement);
		auto* Block = new(Allocator) AST::FCompoundStatement(Allocator, Parser.Scanner.GetCurrentToken()->SourceInfo);
		// To help debug
		AST::FNode* PrevStatement = nullptr;
		while (Parser.Scanner.HasMoreTokens())
		{
			AST::FNode* Statement = nullptr;
			auto Result = ParseStatement(Parser, Allocator, &Statement);
			if (Result == EParseResult::NotMatched)
			{
				if (Parser.Scanner.MatchToken(EHlslToken::RightBrace))
				{
					*OutStatement = Block;
					return EParseResult::Matched;
				}
				else
				{
					Parser.Scanner.SourceError(TEXT("Statement expected!"));
					break;
				}
			}
			else if (Result == EParseResult::Error)
			{
				break;
			}

			if (Statement)
			{
				Block->Statements.Add(Statement);
			}

			PrevStatement = Statement;
		}

		Parser.Scanner.SourceError(TEXT("'}' expected!"));
		return ParseResultError();
	}

	bool ParseFunctionStorageSpecifiers(FHlslParser& Parser, bool& bOutInline, bool& bOutStatic)
	{
		bOutInline = false;
		bOutStatic = false;
		while (true)
		{
			if (Parser.Scanner.MatchToken(EHlslToken::Inline))
			{
				if (bOutInline)
				{
					Parser.Scanner.SourceError(TEXT("Duplicate storage qualifier found: 'inline'"));
					return false;
				}
				bOutInline = true;
			}
			else if (Parser.Scanner.MatchToken(EHlslToken::Static))
			{
				if (bOutStatic)
				{
					Parser.Scanner.SourceError(TEXT("Duplicate storage qualifier found: 'static'"));
					return false;
				}
				bOutStatic = true;
			}
			else
			{
				break;
			}
		}
		return true;
	}

	EParseResult ParseFunctionDeclaration(FHlslParser& Parser, FLinearAllocator* Allocator, TLinearArray<AST::FAttribute*>& Attributes, AST::FNode** OutFunction)
	{
		const auto* CurrentToken = Parser.Scanner.GetCurrentToken();

		// Inline could be used but will be ignored per the hlsl spec. If found then this HAS to be a function.
		bool bFoundInline = false, bFoundStatic = false;
		if (!ParseFunctionStorageSpecifiers(Parser, bFoundInline, bFoundStatic))
		{
			return ParseResultError();
		}

		// Symbol scope to be used for function parameters and body, but not the function name itself.
		// We use TOptional here because FCreateSymbolScope destructor should run here (after finishing parsing the body), 
		// but it will be constructed inside ParseFunctionDeclarator.
		TOptional<FCreateSymbolScope> FunctionSymbolScope;

		AST::FFunction* Function = nullptr;
		EParseResult Result = ParseFunctionDeclarator(Parser, Allocator, &Function, &FunctionSymbolScope);
		if (Result == EParseResult::NotMatched)
		{
			if (bFoundInline)
			{
				Parser.Scanner.SourceError(TEXT("Function declaration expected ('inline' keyword found)"));
				return EParseResult::Error;
			}

			return EParseResult::NotMatched;
		}
		else if( Result == EParseResult::Error)
		{
			return Result;
		}
		Function->bIsStatic = bFoundStatic;

		if (Parser.Scanner.MatchToken(EHlslToken::Semicolon))
		{
			// Forward declaration
			Function->bIsDefinition = true;
			*OutFunction = Function;
			return EParseResult::Matched;
		}
		else
		{
			// Optional semantic
			if (Parser.Scanner.MatchToken(EHlslToken::Colon))
			{
				const auto* Semantic = Parser.Scanner.GetCurrentToken();
				if (!Parser.Scanner.MatchToken(EHlslToken::Identifier))
				{
					Parser.Scanner.SourceError(TEXT("Identifier for semantic expected"));
					return ParseResultError();
				}
				Function->ReturnSemantic = new(Allocator) AST::FSemanticSpecifier(Allocator, *Semantic->String, Semantic->SourceInfo);
			}

			if (!Parser.Scanner.MatchToken(EHlslToken::LeftBrace))
			{
				Parser.Scanner.SourceError(TEXT("'{' expected"));
				return ParseResultError();
			}

			if (Attributes.Num() > 0)
			{
				Function->Attributes = Attributes;
			}

			auto* FunctionDefinition = new(Allocator) AST::FFunctionDefinition(Allocator, CurrentToken->SourceInfo);
			AST::FNode* Body = nullptr;
			Result = ParseStatementBlock(Parser, Allocator, &Body);
			if (Result == EParseResult::Matched)
			{
				FunctionDefinition->Body = (AST::FCompoundStatement*)Body;
				FunctionDefinition->Prototype = Function;
				*OutFunction = FunctionDefinition;
				
				// Adding function declarations to symbols defined within specific namespace
				if (Parser.CurrentScope && Parser.CurrentScope->Name != nullptr)
				{
					Parser.CurrentScope->Symbols.Add(FunctionDefinition->Prototype->Identifier->ToString());
				}
			}
		}

		return Result;
	}

	EParseResult ParseLocalDeclaration(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutDeclaration)
	{
		// We don't support namespace declarations in Local scopes, only in Global scopes. Local declarations that start with <namespace>:: are not allowed in language syntax
		const FHlslToken* Token = Parser.Scanner.PeekToken();
		const FHlslToken* Token2 = Parser.Scanner.PeekToken(1);
		if (Token && !Token->String.IsEmpty() && Token2 && Token2->Token == EHlslToken::ColonColon)
		{
			const FSymbolScope* Namespace = Parser.CurrentScope->GetGlobalScope();
			Namespace = Namespace->FindNamespace(*Token->String);
			if (Namespace)
			{
				return EParseResult::NotMatched;
			}
		}

		AST::FDeclaratorList* List = nullptr;
		auto Result = ParseGeneralDeclaration(
			Parser, Parser.CurrentScope, Allocator, &List, 0,
			EDF_CONST_ROW_MAJOR | EDF_TEXTURE_SAMPLER_OR_BUFFER | EDF_INITIALIZER | EDF_INITIALIZER_LIST | EDF_SEMICOLON | EDF_MULTIPLE | EDF_STATIC
		);
		*OutDeclaration = List;
		return Result;
	}

	EParseResult ParseStaticAssertStatement(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutDeclaration)
	{
		const FHlslToken* StaticAssertToken = Parser.Scanner.PeekToken();
		if (!Parser.Scanner.MatchToken(EHlslToken::StaticAssert))
		{
			return EParseResult::NotMatched;
		}

		check(StaticAssertToken);

		// Store statement keyword to support _Static_assert (HLSL) and static_assert (C++11 like compilers)
		AST::FStaticAssertStatement* Statement = new(Allocator) AST::FStaticAssertStatement(Allocator, StaticAssertToken->SourceInfo, StaticAssertToken->String);

		if (!Parser.Scanner.MatchToken(EHlslToken::LeftParenthesis))
		{
			Parser.Scanner.SourceError(TEXT("'(' expected!\n"));
			return ParseResultError();
		}

		// Parse assertion conditional expression
		if (ParseExpression(Parser.Scanner, Parser.CurrentScope, 0, Allocator, &Statement->Condition) != EParseResult::Matched)
		{
			Parser.Scanner.SourceError(TEXT("Expression expected"));
			return ParseResultError();
		}

		// Parse message string literal
		if (!Parser.Scanner.MatchToken(EHlslToken::Comma))
		{
			Parser.Scanner.SourceError(TEXT("',' expected!\n"));
			return ParseResultError();
		}

		const FHlslToken* MessageToken = Parser.Scanner.PeekToken();
		if (!Parser.Scanner.MatchToken(EHlslToken::StringConstant))
		{
			Parser.Scanner.SourceError(TEXT("')' expected!\n"));
			return ParseResultError();
		}

		Statement->Message = MessageToken->String;

		if (!Parser.Scanner.MatchToken(EHlslToken::RightParenthesis))
		{
			Parser.Scanner.SourceError(TEXT("')' expected!\n"));
			return ParseResultError();
		}

		if (!Parser.Scanner.MatchToken(EHlslToken::Semicolon))
		{
			Parser.Scanner.SourceError(TEXT("';' expected!\n"));
			return ParseResultError();
		}

		*OutDeclaration = Statement;

		return EParseResult::Matched;
	}

	EParseResult ParseC99PragmaStatementBody(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutDeclaration)
	{
		if (!Parser.Scanner.MatchToken(EHlslToken::LeftParenthesis))
		{
			Parser.Scanner.SourceError(TEXT("'(' expected!\n"));
			return ParseResultError();
		}

		const FHlslToken* PragmaArgumentToken = Parser.Scanner.PeekToken();
		if (!Parser.Scanner.MatchToken(EHlslToken::StringConstant))
		{
			Parser.Scanner.SourceError(TEXT("string constant expected!\n"));
			return ParseResultError();
		}
		// Store the string inside the pragma so we can round-trip it properly
		AST::FC99PragmaStatement* Statement = new(Allocator) AST::FC99PragmaStatement(Allocator, PragmaArgumentToken->SourceInfo, PragmaArgumentToken->String);

		if (!Parser.Scanner.MatchToken(EHlslToken::RightParenthesis))
		{
			Parser.Scanner.SourceError(TEXT("')' expected!\n"));
			return ParseResultError();
		}

		*OutDeclaration = Statement;

		return EParseResult::Matched;
	}

	EParseResult ParseC99PragmaStatement(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutDeclaration)
	{
		if (!Parser.Scanner.MatchToken(EHlslToken::C99Pragma))
		{
			return EParseResult::NotMatched;
		}
		return ParseC99PragmaStatementBody(Parser, Allocator, OutDeclaration);
	}

	EParseResult ParseGlobalVariableDeclaration(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutDeclaration)
	{
		AST::FDeclaratorList* List = nullptr;
		auto Result = ParseGeneralDeclaration(
			Parser, Parser.CurrentScope, Allocator, &List, ETF_USER_TYPES | ETF_ERROR_IF_NOT_USER_TYPE,
			EDF_CONST_ROW_MAJOR | EDF_STATIC | EDF_SHARED | EDF_TEXTURE_SAMPLER_OR_BUFFER | EDF_INITIALIZER | EDF_INITIALIZER_LIST | EDF_SEMICOLON | EDF_MULTIPLE | EDF_UNIFORM | EDF_INTERPOLATION | EDF_REGISTER | EDF_PACKOFFSET
		);
		*OutDeclaration = List;
		return Result;
	}

	EParseResult ParseReturnStatement(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutStatement)
	{
		auto* Statement = new(Allocator) AST::FJumpStatement(Allocator, AST::EJumpType::Return, Parser.Scanner.GetCurrentToken()->SourceInfo);

		if (Parser.Scanner.MatchToken(EHlslToken::Semicolon))
		{
			*OutStatement = Statement;
			return EParseResult::Matched;
		}

		if (ParseExpression(Parser.Scanner, Parser.CurrentScope, EEF_ALLOW_ASSIGNMENT, Allocator, &Statement->OptionalExpression) != EParseResult::Matched)
		{
			Parser.Scanner.SourceError(TEXT("Expression expected"));
			return ParseResultError();
		}

		if (!Parser.Scanner.MatchToken(EHlslToken::Semicolon))
		{
			Parser.Scanner.SourceError(TEXT("';' expected"));
			return ParseResultError();
		}

		*OutStatement = Statement;
		return EParseResult::Matched;
	}

	EParseResult ParseDoStatement(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutStatement)
	{
		FCreateSymbolScope SymbolScope(Allocator, &Parser.CurrentScope, FSymbolScope::EType::Do);
		const auto* Token = Parser.Scanner.GetCurrentToken();
		AST::FNode* Body = nullptr;
		auto Result = ParseStatement(Parser, Allocator, &Body);
		if (Result != EParseResult::Matched)
		{
			return ParseResultError();
		}

		if (!Parser.Scanner.MatchToken(EHlslToken::While))
		{
			Parser.Scanner.SourceError(TEXT("'while' expected"));
			return ParseResultError();
		}

		if (!Parser.Scanner.MatchToken(EHlslToken::LeftParenthesis))
		{
			Parser.Scanner.SourceError(TEXT("'(' expected"));
			return ParseResultError();
		}

		AST::FExpression* ConditionExpression = nullptr;
		if (ParseExpression(Parser.Scanner, Parser.CurrentScope, EEF_ALLOW_ASSIGNMENT, Allocator, &ConditionExpression) != EParseResult::Matched)
		{
			Parser.Scanner.SourceError(TEXT("Expression expected"));
			return ParseResultError();
		}

		if (!Parser.Scanner.MatchToken(EHlslToken::RightParenthesis))
		{
			Parser.Scanner.SourceError(TEXT("')' expected"));
			return ParseResultError();
		}

		if (!Parser.Scanner.MatchToken(EHlslToken::Semicolon))
		{
			Parser.Scanner.SourceError(TEXT("';' expected"));
			return ParseResultError();
		}

		auto* DoWhile = new(Allocator) AST::FIterationStatement(Allocator, Token->SourceInfo, AST::EIterationType::DoWhile);
		DoWhile->Condition = ConditionExpression;
		DoWhile->Body = Body;
		*OutStatement = DoWhile;
		return EParseResult::Matched;
	}

	EParseResult ParseWhileStatement(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutStatement)
	{
		FCreateSymbolScope SymbolScope(Allocator, &Parser.CurrentScope, FSymbolScope::EType::While);
		const auto* Token = Parser.Scanner.GetCurrentToken();
		if (!Parser.Scanner.MatchToken(EHlslToken::LeftParenthesis))
		{
			Parser.Scanner.SourceError(TEXT("'(' expected"));
			return ParseResultError();
		}

		AST::FExpression* ConditionExpression = nullptr;
		if (ParseExpression(Parser.Scanner, Parser.CurrentScope, EEF_ALLOW_ASSIGNMENT, Allocator, &ConditionExpression) != EParseResult::Matched)
		{
			Parser.Scanner.SourceError(TEXT("Expression expected"));
			return ParseResultError();
		}

		if (!Parser.Scanner.MatchToken(EHlslToken::RightParenthesis))
		{
			Parser.Scanner.SourceError(TEXT("')' expected"));
			return ParseResultError();
		}

		AST::FNode* Body = nullptr;
		auto Result = ParseStatement(Parser, Allocator, &Body);
		if (Result != EParseResult::Matched)
		{
			return ParseResultError();
		}

		auto* While = new(Allocator) AST::FIterationStatement(Allocator, Token->SourceInfo, AST::EIterationType::While);
		While->Condition = ConditionExpression;
		While->Body = Body;
		*OutStatement = While;
		return EParseResult::Matched;
	}

	EParseResult ParseForStatement(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutStatement)
	{
		FCreateSymbolScope SymbolScope(Allocator, &Parser.CurrentScope, FSymbolScope::EType::For);
		const auto* Token = Parser.Scanner.GetCurrentToken();
		if (!Parser.Scanner.MatchToken(EHlslToken::LeftParenthesis))
		{
			Parser.Scanner.SourceError(TEXT("Expected '('!\n"));
			return ParseResultError();
		}

		AST::FNode* InitExpression = nullptr;
		if (Parser.Scanner.MatchToken(EHlslToken::Semicolon))
		{
			// Do nothing!
		}
		else
		{
			auto OriginalToken = Parser.Scanner.GetCurrentTokenIndex();
			auto Result = ParseLocalDeclaration(Parser, Allocator, &InitExpression);
			if (Result == EParseResult::Error)
			{
				Parser.Scanner.SourceError(TEXT("Expected expression or declaration!\n"));
				return ParseResultError();
			}
			else if (Result == EParseResult::NotMatched)
			{
				Parser.Scanner.SetCurrentTokenIndex(OriginalToken);
				AST::FExpression* Expr = nullptr;
				Result = ParseExpressionList2(Parser.Scanner, Parser.CurrentScope, Allocator, AST::FExpressionList::EType::FreeForm, &Expr);
				if (Result == EParseResult::Error)
				{
					Parser.Scanner.SourceError(TEXT("Expected expression or declaration!\n"));
					return ParseResultError();
				}
				InitExpression = Expr;

				AST::FExpression* ConditionExpression = nullptr;
				if (!Parser.Scanner.MatchToken(EHlslToken::Semicolon))
				{
					Parser.Scanner.SourceError(TEXT("Expected ';'!\n"));
					return ParseResultError();
				}
			}
		}

		AST::FExpression* ConditionExpression = nullptr;
		if (Parser.Scanner.MatchToken(EHlslToken::Semicolon))
		{
			// Do nothing!
		}
		else
		{
			auto Result = ParseExpressionList2(Parser.Scanner, Parser.CurrentScope, Allocator, AST::FExpressionList::EType::FreeForm, &ConditionExpression);
			if (Result == EParseResult::Error)
			{
				Parser.Scanner.SourceError(TEXT("Expected expression or declaration!\n"));
				return ParseResultError();
			}

			if (!Parser.Scanner.MatchToken(EHlslToken::Semicolon))
			{
				Parser.Scanner.SourceError(TEXT("Expected ';'!\n"));
				return ParseResultError();
			}
		}

		AST::FExpression* RestExpression = nullptr;
		if (Parser.Scanner.MatchToken(EHlslToken::RightParenthesis))
		{
			// Do nothing!
		}
		else
		{
			auto Result = ParseExpressionList2(Parser.Scanner, Parser.CurrentScope, Allocator, AST::FExpressionList::EType::FreeForm, &RestExpression);
			if (Result == EParseResult::Error)
			{
				Parser.Scanner.SourceError(TEXT("Expected expression or declaration!\n"));
				return ParseResultError();
			}

			if (!Parser.Scanner.MatchToken(EHlslToken::RightParenthesis))
			{
				Parser.Scanner.SourceError(TEXT("Expected ')'!\n"));
				return ParseResultError();
			}
		}

		AST::FNode* Body = nullptr;
		auto Result = ParseStatement(Parser, Allocator, &Body);
		if (Result != EParseResult::Matched)
		{
			return ParseResultError();
		}

		auto* For = new(Allocator) AST::FIterationStatement(Allocator, Token->SourceInfo, AST::EIterationType::For);
		For->InitStatement = InitExpression;
		For->Condition = ConditionExpression;
		For->RestExpression = RestExpression;
		For->Body = Body;
		*OutStatement = For;
		return EParseResult::Matched;
	}

	EParseResult ParseIfStatement(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutStatement)
	{
		FCreateSymbolScope SymbolScope(Allocator, &Parser.CurrentScope, FSymbolScope::EType::If);

		auto* Statement = new(Allocator) AST::FSelectionStatement(Allocator, Parser.Scanner.GetCurrentToken()->SourceInfo);

		if (!Parser.Scanner.MatchToken(EHlslToken::LeftParenthesis))
		{
			Parser.Scanner.SourceError(TEXT("'(' expected"));
			return ParseResultError();
		}

		if (ParseExpression(Parser.Scanner, Parser.CurrentScope, EEF_ALLOW_ASSIGNMENT, Allocator, &Statement->Condition) != EParseResult::Matched)
		{
			Parser.Scanner.SourceError(TEXT("Expression expected"));
			return ParseResultError();
		}

		if (!Parser.Scanner.MatchToken(EHlslToken::RightParenthesis))
		{
			Parser.Scanner.SourceError(TEXT("')' expected"));
			return ParseResultError();
		}

		if (ParseStatement(Parser, Allocator, &Statement->ThenStatement) != EParseResult::Matched)
		{
			Parser.Scanner.SourceError(TEXT("Statement expected"));
			return ParseResultError();
		}

		if (Parser.Scanner.MatchToken(EHlslToken::Else))
		{
			if (ParseStatement(Parser, Allocator, &Statement->ElseStatement) != EParseResult::Matched)
			{
				Parser.Scanner.SourceError(TEXT("Statement expected"));
				return ParseResultError();
			}
		}

		*OutStatement = Statement;
		return EParseResult::Matched;
	}

	EParseResult ParseAttributeArgList(FHlslScanner& Scanner, FSymbolScope* SymbolScope, FLinearAllocator* Allocator, AST::FAttribute* OutAttribute)
	{
		while (Scanner.HasMoreTokens())
		{
			const auto* Token = Scanner.PeekToken();
			if (Scanner.MatchToken(EHlslToken::RightParenthesis))
			{
				return EParseResult::Matched;
			}

			bool bMultiple = false;
			do
			{
				bMultiple = false;
				Token = Scanner.PeekToken();
				if (Scanner.MatchToken(EHlslToken::StringConstant))
				{
					auto* Arg = new(Allocator) AST::FAttributeArgument(Allocator, Token->SourceInfo);
					Arg->StringArgument = Allocator->Strdup(Token->String);
					OutAttribute->Arguments.Add(Arg);
				}
				else
				{
					AST::FExpression* Expression = nullptr;
					EParseResult Result = ParseExpression(Scanner, SymbolScope, 0, Allocator, &Expression);
					if (Result != EParseResult::Matched)
					{
						Scanner.SourceError(TEXT("Incorrect attribute expression!\n"));
						return ParseResultError();
					}
					auto* Arg = new(Allocator) AST::FAttributeArgument(Allocator, Token->SourceInfo);
					Arg->ExpressionArgument = Expression;
					OutAttribute->Arguments.Add(Arg);
				}

				if (Scanner.MatchToken(EHlslToken::Comma))
				{
					bMultiple = true;
				}
			}
			while (bMultiple);
		}

		return ParseResultError();
	}

	EParseResult TryParseAttribute(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FAttribute** OutAttribute)
	{
		const auto* Token = Parser.Scanner.GetCurrentToken();
		if (Parser.Scanner.MatchToken(EHlslToken::LeftSquareBracket))
		{
			auto* Identifier = Parser.Scanner.GetCurrentToken();
			if (!Parser.Scanner.MatchToken(EHlslToken::Identifier))
			{
				Parser.Scanner.SourceError(TEXT("Incorrect attribute\n"));
				return ParseResultError();
			}

			auto* Attribute = new(Allocator) AST::FAttribute(Allocator, Token->SourceInfo, Allocator->Strdup(Identifier->String));

			if (Parser.Scanner.MatchToken(EHlslToken::LeftParenthesis))
			{
				auto Result = ParseAttributeArgList(Parser.Scanner, Parser.CurrentScope, Allocator, Attribute);
				if (Result != EParseResult::Matched)
				{
					Parser.Scanner.SourceError(TEXT("Incorrect attribute! Expected ')'.\n"));
					return ParseResultError();
				}
			}

			if (!Parser.Scanner.MatchToken(EHlslToken::RightSquareBracket))
			{
				Parser.Scanner.SourceError(TEXT("Incorrect attribute\n"));
				return ParseResultError();
			}

			*OutAttribute = Attribute;
			return EParseResult::Matched;
		}

		return EParseResult::NotMatched;
	}

	EParseResult ParseSwitchBody(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FSwitchBody** OutBody)
	{
		const auto* Token = Parser.Scanner.GetCurrentToken();

		if (!Parser.Scanner.MatchToken(EHlslToken::LeftBrace))
		{
			Parser.Scanner.SourceError(TEXT("'{' expected"));
			return ParseResultError();
		}

		auto* Body = new(Allocator) AST::FSwitchBody(Allocator, Token->SourceInfo);

		// Empty switch
		if (Parser.Scanner.MatchToken(EHlslToken::RightBrace))
		{
			*OutBody = Body;
			return EParseResult::Matched;
		}

		Body->CaseList = new(Allocator) AST::FCaseStatementList(Allocator, Token->SourceInfo);

		bool bDefaultFound = false;
		while (Parser.Scanner.HasMoreTokens())
		{
			Token = Parser.Scanner.GetCurrentToken();
			if (Parser.Scanner.MatchToken(EHlslToken::RightBrace))
			{
				break;
			}

			auto* Labels = new(Allocator) AST::FCaseLabelList(Allocator, Token->SourceInfo);
			auto* CaseStatement = new(Allocator) AST::FCaseStatement(Allocator, Token->SourceInfo, Labels);

			// Case labels
			bool bLabelFound = false;
			do
			{
				bLabelFound = false;
				AST::FCaseLabel* Label = nullptr;
				Token = Parser.Scanner.GetCurrentToken();
				if (Parser.Scanner.MatchToken(EHlslToken::Default))
				{
					if (bDefaultFound)
					{
						Parser.Scanner.SourceError(TEXT("'default' found twice on switch() statement!"));
						return ParseResultError();
					}

					if (!Parser.Scanner.MatchToken(EHlslToken::Colon))
					{
						Parser.Scanner.SourceError(TEXT("':' expected"));
						return ParseResultError();
					}

					Label = new(Allocator) AST::FCaseLabel(Allocator, Token->SourceInfo, nullptr);
					bDefaultFound = true;
					bLabelFound = true;
				}
				else if (Parser.Scanner.MatchToken(EHlslToken::Case))
				{
					AST::FExpression* CaseExpression = nullptr;
					if (ParseExpression(Parser.Scanner, Parser.CurrentScope, EEF_ALLOW_ASSIGNMENT, Allocator, &CaseExpression) != EParseResult::Matched)
					{
						Parser.Scanner.SourceError(TEXT("Expression expected on case label!"));
						return ParseResultError();
					}

					if (!Parser.Scanner.MatchToken(EHlslToken::Colon))
					{
						Parser.Scanner.SourceError(TEXT("':' expected"));
						return ParseResultError();
					}

					Label = new(Allocator) AST::FCaseLabel(Allocator, Token->SourceInfo, CaseExpression);
					bLabelFound = true;
				}

				if (Label)
				{
					CaseStatement->Labels->Labels.Add(Label);
				}
			}
			while (bLabelFound);

			// Statements
			Token = Parser.Scanner.GetCurrentToken();
			bool bMatchedOnce = false;
			while (Parser.Scanner.HasMoreTokens())
			{
				auto* Peek = Parser.Scanner.PeekToken();
				if (!Peek)
				{
					break;
				}
				else if (Peek->Token == EHlslToken::RightBrace)
				{
					// End of switch
					break;
				}
				else if (Peek->Token == EHlslToken::Case || Peek->Token == EHlslToken::Default)
				{
					// Next CaseStatement
					break;
				}
				else
				{
					AST::FNode* Statement = nullptr;
					auto Result = ParseStatement(Parser, Allocator, &Statement);
					if (Result == EParseResult::Error)
					{
						return ParseResultError();
					}
					else if (Result == EParseResult::NotMatched)
					{
						Parser.Scanner.SourceError(TEXT("Internal Error parsing statment inside case list"));
						return ParseResultError();
					}
					// Ignore null statements, e.g. from a double semicolon `;;`
					else if (Statement != nullptr)
					{
						CaseStatement->Statements.Add(Statement);
					}
				}
			}

			Body->CaseList->Cases.Add(CaseStatement);
		}

		*OutBody = Body;
		return EParseResult::Matched;
	}

	EParseResult ParseSwitchStatement(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutStatement)
	{
		const auto* Token = Parser.Scanner.GetCurrentToken();
		if (!Parser.Scanner.MatchToken(EHlslToken::LeftParenthesis))
		{
			Parser.Scanner.SourceError(TEXT("'(' expected"));
			return ParseResultError();
		}

		AST::FExpression* Condition = nullptr;
		if (ParseExpression(Parser.Scanner, Parser.CurrentScope, 0, Allocator, &Condition) != EParseResult::Matched)
		{
			Parser.Scanner.SourceError(TEXT("Expression expected"));
			return ParseResultError();
		}

		if (!Parser.Scanner.MatchToken(EHlslToken::RightParenthesis))
		{
			Parser.Scanner.SourceError(TEXT("')' expected"));
			return ParseResultError();
		}

		AST::FSwitchBody* Body = nullptr;
		if (ParseSwitchBody(Parser, Allocator, &Body) != EParseResult::Matched)
		{
			return ParseResultError();
		}

		auto* Switch = new(Allocator) AST::FSwitchStatement(Allocator, Token->SourceInfo, Condition, Body);
		*OutStatement = Switch;

		return EParseResult::Matched;
	}

	EParseResult ParseExpressionStatement(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutStatement)
	{
		auto OriginalToken = Parser.Scanner.GetCurrentTokenIndex();
		auto* Statement = new(Allocator) AST::FExpressionStatement(Allocator, nullptr, Parser.Scanner.GetCurrentToken()->SourceInfo);
		if (ParseExpression(Parser.Scanner, Parser.CurrentScope, EEF_ALLOW_ASSIGNMENT, Allocator, &Statement->Expression) == EParseResult::Matched)
		{
			// Adding Comma to handle free form expressions that can sometimes be generated from blueprints. We could adjust RulesStatements and swap ParseExpressionStatement there with ParseExpressionList2 but this seems to work as well and less code changes
			if (Parser.Scanner.MatchToken(EHlslToken::Semicolon) || Parser.Scanner.MatchToken(EHlslToken::Comma))
			{
				*OutStatement = Statement;
				return EParseResult::Matched;
			}
		}

		Parser.Scanner.SetCurrentTokenIndex(OriginalToken);
		return EParseResult::NotMatched;
	}

	EParseResult ParseEmptyStatement(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutStatement)
	{
		check(*OutStatement == nullptr);

		// Nothing to do here...
		return EParseResult::Matched;
	}

	EParseResult ParseBreakStatement(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutStatement)
	{
		check(*OutStatement == nullptr);

		auto* Statement = new(Allocator) AST::FJumpStatement(Allocator, AST::EJumpType::Break, Parser.Scanner.PeekToken(-1)->SourceInfo);

		if (Parser.Scanner.MatchToken(EHlslToken::Semicolon))
		{
			*OutStatement = Statement;
			return EParseResult::Matched;
		}

		return ParseResultError();
	}

	EParseResult ParseContinueStatement(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutStatement)
	{
		check(*OutStatement == nullptr);

		auto* Statement = new(Allocator) AST::FJumpStatement(Allocator, AST::EJumpType::Continue, Parser.Scanner.PeekToken(-1)->SourceInfo);

		if (Parser.Scanner.MatchToken(EHlslToken::Semicolon))
		{
			*OutStatement = Statement;
			return EParseResult::Matched;
		}

		return ParseResultError();
	}

	EParseResult ParseTypedef(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutNode)
	{
		if (!Parser.Scanner.MatchToken(EHlslToken::Typedef))
		{
			return EParseResult::NotMatched;
		}

		EParseResult Result = ParseGlobalVariableDeclaration(Parser, Allocator, OutNode);
		if (Result == EParseResult::Matched)
		{
			if (*OutNode)
			{
				CrossCompiler::AST::FDeclaratorList* DeclList = (*OutNode)->AsDeclaratorList();
				if (DeclList)
				{
					DeclList->bTypedef = true;
					for (auto* DeclNode : DeclList->Declarations)
					{
						CrossCompiler::AST::FDeclaration* Decl = DeclNode->AsDeclaration();
						if (Decl)
						{
							if (Decl->Identifier && Decl->Identifier->Len())
							{
								Parser.CurrentScope->Add(Decl->Identifier->GetData());
							}
						}
						else
						{
							Parser.Scanner.SourceError(TEXT("Internal error on typedef declaration"));
						}
					}
				}
				else
				{
					Parser.Scanner.SourceError(TEXT("Internal error parsing typedef declarator list"));
					return EParseResult::Error;
				}
			}
			else
			{
				Parser.Scanner.SourceError(TEXT("Internal error parsing typedef with null node"));
				return EParseResult::Error;
			}
		}

		return EParseResult::Matched;
	}

	bool MatchConstantBufferNoTemplate(FHlslParser& Parser)
	{
		const FHlslToken* Peek = Parser.Scanner.PeekToken();
		if (Peek && Peek->Token == EHlslToken::ConstantBuffer)
		{
			const FHlslToken* Peek1 = Parser.Scanner.PeekToken(1);
			if (!Peek1 || Peek1->Token != EHlslToken::Lower)
			{
				// it's a standard one, make sure we match it
				return Parser.Scanner.MatchToken(EHlslToken::ConstantBuffer);
			}
		}

		return false;
	}

	// Prints the specified number of tokens, starting from the token the scanner is currently pointing to, into a string.
	static FString PrintDiagnosticTokenStream(const FHlslScanner& Scanner, int32 Count)
	{
		const FHlslToken* Token = nullptr;
		FString OutString;
		for (int32 TokenIndex = 0; TokenIndex < Count && (Token = Scanner.PeekToken(TokenIndex)) != nullptr; ++TokenIndex)
		{
			if (TokenIndex > 0)
			{
				OutString += TCHAR(' ');
			}
			OutString += Token->String;
		}
		return OutString;
	}

	EParseResult TryTranslationUnit(FHlslParser& Parser, FLinearAllocator* Allocator, AST::FNode** OutNode)
	{
		if (MatchPragma(Parser, Allocator, OutNode))
		{
			return EParseResult::Matched;
		}

		if (Parser.Scanner.MatchToken(EHlslToken::CBuffer) || MatchConstantBufferNoTemplate(Parser))
		{
			auto Result = ParseCBuffer(Parser, Allocator, OutNode);
			if (Result == EParseResult::Error || Result == EParseResult::Matched)
			{
				return Result;
			}
		}

		// Match Attributes
		TLinearArray<AST::FAttribute*> Attributes(Allocator);
		while (Parser.Scanner.HasMoreTokens())
		{
			const auto* Peek = Parser.Scanner.GetCurrentToken();
			if (Peek->Token == EHlslToken::LeftSquareBracket)
			{
				AST::FAttribute* Attribute = nullptr;
				auto Result = TryParseAttribute(Parser, Allocator, &Attribute);
				if (Result == EParseResult::Matched)
				{
					Attributes.Add(Attribute);
					continue;
				}
				else if (Result == EParseResult::Error)
				{
					return ParseResultError();
				}
			}

			break;
		}

		const CrossCompiler::FHlslToken* Peek = Parser.Scanner.GetCurrentToken();
		if (!Peek)
		{
			return ParseResultError();
		}
		
		auto Result = ParseTypedef(Parser, Allocator, OutNode);
		if (Result == EParseResult::Error || Result == EParseResult::Matched)
		{
			return Result;
		}

		Result = ParseFunctionDeclaration(Parser, Allocator, Attributes, OutNode);
		if (Result == EParseResult::Error || Result == EParseResult::Matched)
		{
			return Result;
		}

		Result = ParseGlobalVariableDeclaration(Parser, Allocator, OutNode);
		if (Result == EParseResult::Error || Result == EParseResult::Matched)
		{
			return Result;
		}

		Result = ParseStaticAssertStatement(Parser, Allocator, OutNode);
		if (Result == EParseResult::Error || Result == EParseResult::Matched)
		{
			return Result;
		}

		Result = ParseC99PragmaStatement(Parser, Allocator, OutNode);
		if (Result == EParseResult::Error || Result == EParseResult::Matched)
		{
			return Result;
		}

		Result = ParseNamespaceBlock(Parser, Allocator, OutNode);
		if (Result == EParseResult::Error || Result == EParseResult::Matched)
		{
			return Result;
		}

		// Ignore semicolons with no code before them.
		if (Parser.Scanner.MatchToken(EHlslToken::Semicolon))
		{
			return EParseResult::Ignore;
		}

		constexpr int32 kTokenLookAheadForError = 10;
		Parser.Scanner.SourceError(FString::Printf(TEXT("Unable to match rule! [Token stream: \"%s\"]"), *PrintDiagnosticTokenStream(Parser.Scanner, kTokenLookAheadForError)));
		return ParseResultError();
	}

	namespace ParserRules
	{
		static struct FStaticInitializer
		{
			FStaticInitializer()
			{
				RulesStatements.Add(FRulePair(EHlslToken::C99Pragma, ParseC99PragmaStatementBody));
				RulesStatements.Add(FRulePair(EHlslToken::LeftBrace, ParseStatementBlock));
				RulesStatements.Add(FRulePair(EHlslToken::Return, ParseReturnStatement));
				RulesStatements.Add(FRulePair(EHlslToken::Do, ParseDoStatement));
				RulesStatements.Add(FRulePair(EHlslToken::While, ParseWhileStatement, true));
				RulesStatements.Add(FRulePair(EHlslToken::For, ParseForStatement, true));
				RulesStatements.Add(FRulePair(EHlslToken::If, ParseIfStatement, true));
				RulesStatements.Add(FRulePair(EHlslToken::Switch, ParseSwitchStatement, true));
				RulesStatements.Add(FRulePair(EHlslToken::Semicolon, ParseEmptyStatement));
				RulesStatements.Add(FRulePair(EHlslToken::Break, ParseBreakStatement));
				RulesStatements.Add(FRulePair(EHlslToken::Continue, ParseContinueStatement));
				RulesStatements.Add(FRulePair(EHlslToken::Invalid, ParseLocalDeclaration, true));
				// Always try expressions last
				RulesStatements.Add(FRulePair(EHlslToken::Invalid, ParseExpressionStatement));
			}
		} GStaticInitializer;
	}

	FHlslParser::FHlslParser(FLinearAllocator* InAllocator, FCompilerMessages& InCompilerMessages, TConstArrayView<UE::HlslParser::FScopedDeclarations> InScopedDeclarations) :
		Scanner(InCompilerMessages),
		CompilerMessages(InCompilerMessages),
		GlobalScope(InAllocator, nullptr, FSymbolScope::EType::Global),
		Namespaces(InAllocator, nullptr, FSymbolScope::EType::Namespace),
		Allocator(InAllocator)
	{
		CurrentScope = &GlobalScope;

		for (const UE::HlslParser::FScopedDeclarations& ScopedDeclarations : InScopedDeclarations)
		{
			TArray<FCreateSymbolScope> ScopeStack;

			FSymbolScope* IterScope = CurrentScope;
			for (FStringView Scope : ScopedDeclarations.Scope)
			{
				ScopeStack.Emplace(Allocator, &IterScope, FSymbolScope::EType::ScopedDeclaration);
				IterScope->Name = Scope.GetData();
			}

			for (FStringView Symbol : ScopedDeclarations.Symbols)
			{
				IterScope->Add(Symbol);
			}
		}

		// Register built-in structure for DXR in SM6
		CurrentScope->Add(TEXT("RayDesc"));
	}

	namespace Parser
	{
		bool Parse(const FString& Input, const FString& Filename, FCompilerMessages& OutCompilerMessages, TConstArrayView<UE::HlslParser::FScopedDeclarations> InScopedDeclarations, TCallback* Callback, void* CallbackData)
		{
			CrossCompiler::FNodeContainer Container;

			const bool bSuccess = Parse(Input, Filename, OutCompilerMessages, InScopedDeclarations, Container);

			if (bSuccess && Callback)
			{
				Callback(CallbackData, &Container.Allocator, Container.Nodes);
			}

			return bSuccess;
		}

		bool Parse(const FString& Input, const FString& Filename, FCompilerMessages& OutCompilerMessages, TConstArrayView<UE::HlslParser::FScopedDeclarations> InScopedDeclarations, TFunction< void(CrossCompiler::FLinearAllocator* Allocator, CrossCompiler::TLinearArray<CrossCompiler::AST::FNode*>& ASTNodes)> Function)
		{
			CrossCompiler::FNodeContainer Container;

			const bool bSuccess = Parse(Input, Filename, OutCompilerMessages, InScopedDeclarations, Container);

			if (bSuccess)
			{
				Function(&Container.Allocator, Container.Nodes);
			}

			return bSuccess;
		}

		bool Parse(const FString& Input, const FString& Filename, FCompilerMessages& OutCompilerMessages, TConstArrayView<UE::HlslParser::FScopedDeclarations> InScopedDeclarations, CrossCompiler::FNodeContainer& OutContainer)
		{
			FHlslParser Parser(&OutContainer.Allocator, OutCompilerMessages, InScopedDeclarations);
			if (!Parser.Scanner.Lex(Input, Filename))
			{
				return false;
			}

			bool bSuccess = true;

			AST::FNode* PrevNode = nullptr;
			while (Parser.Scanner.HasMoreTokens())
			{
				auto LastIndex = Parser.Scanner.GetCurrentTokenIndex();
				static FString GlobalDeclOrDefinition(TEXT("Global declaration or definition"));
				AST::FNode* Node = nullptr;
				auto Result = TryTranslationUnit(Parser, &OutContainer.Allocator, &Node);
				if (Result == EParseResult::Error)
				{
					bSuccess = false;
					break;
				}
				else if (Result == EParseResult::Ignore)
				{
					continue;
				}
				else
				{
					check(Result == EParseResult::Matched);
					checkf(Node, TEXT("Null AST Node! PrevNode @ %s(%d)"),
						((PrevNode && PrevNode->SourceInfo.Filename) ? *(*PrevNode->SourceInfo.Filename) : TEXT("<none>")),
						(PrevNode ? PrevNode->SourceInfo.Line : 0));
					OutContainer.Nodes.Add(Node);
				}

				check(LastIndex != Parser.Scanner.GetCurrentTokenIndex());
				PrevNode = Node;
			}

			return bSuccess;
		}

		void WriteNodesToString(void* OutFStringPointer, CrossCompiler::FLinearAllocator* Allocator, CrossCompiler::TLinearArray<CrossCompiler::AST::FNode*>& ASTNodes)
		{
			check(OutFStringPointer);
			FString& OutGeneratedCode = *(FString*)OutFStringPointer;
			CrossCompiler::AST::FASTWriter Writer(OutGeneratedCode);
			for (auto* Node : ASTNodes)
			{
				Node->Write(Writer);
			}
		}
	}
}
