// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HlslParser.h - Interface for parsing hlsl.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "HlslUtils.h"

class Error;

namespace UE::HlslParser
{
	struct FScopedDeclarations;
}

namespace CrossCompiler
{
	struct FLinearAllocator;
	struct FNodeContainer;

	namespace AST
	{
		class FNode;
	}

	enum class EParseResult
	{
		Matched,
		NotMatched,
		Ignore,
		Error,
	};

	namespace Parser
	{
		typedef void TCallback(void* CallbackData, CrossCompiler::FLinearAllocator* Allocator, CrossCompiler::TLinearArray<CrossCompiler::AST::FNode*>& ASTNodes);

		// Returns true if successfully parsed
		HLSLPARSER_API bool Parse(const FString& Input, const FString& Filename, FCompilerMessages& OutCompilerMessages, TConstArrayView<UE::HlslParser::FScopedDeclarations> InScopedDeclarations, TCallback* Callback, void* CallbackData = nullptr);

		// Returns true if successfully parsed
		HLSLPARSER_API bool Parse(const FString& Input, const FString& Filename, FCompilerMessages& OutCompilerMessages, TConstArrayView<UE::HlslParser::FScopedDeclarations> InScopedDeclarations, TFunction< void(CrossCompiler::FLinearAllocator* Allocator, CrossCompiler::TLinearArray<CrossCompiler::AST::FNode*>& ASTNodes)> Function);

		// Returns true if successfully parsed
		HLSLPARSER_API bool Parse(const FString& Input, const FString& Filename, FCompilerMessages& OutCompilerMessages, TConstArrayView<UE::HlslParser::FScopedDeclarations> InScopedDeclarations, CrossCompiler::FNodeContainer& OutContainer);

		// Sample callback to write out all nodes into a string; pass a valid pointer FString* as OutFStringPointer
		HLSLPARSER_API void WriteNodesToString(void* OutFStringPointer, CrossCompiler::FLinearAllocator* Allocator, CrossCompiler::TLinearArray<CrossCompiler::AST::FNode*>& ASTNodes);
	}
}
