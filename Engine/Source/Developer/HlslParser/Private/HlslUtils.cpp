// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HlslUtils.cpp - Utils for HLSL.
=============================================================================*/

#include "HlslUtils.h"
#include "Misc/ScopeLock.h"
#include "HlslAST.h"
#include "HlslParser.h"
#include "HlslParserInternal.h"

#if WITH_LOW_LEVEL_TESTS
#include "Tests/TestHarnessAdapter.h"
#endif // WITH_LOW_LEVEL_TESTS

static bool bLeaveAllUsed = false;

namespace CrossCompiler
{
	namespace Memory
	{
#if USE_PAGE_POOLING
		static struct FPagePoolInstance
		{
			~FPagePoolInstance()
			{
				check(UsedPages.Num() == 0);
				for (int32 Index = 0; Index < FreePages.Num(); ++Index)
				{
					delete FreePages[Index];
				}
			}

			FPage* AllocatePage(SIZE_T PageSize)
			{
				FScopeLock ScopeLock(&CriticalSection);

				if (FreePages.Num() == 0)
				{
					FreePages.Add(new FPage(PageSize));
				}

				auto* Page = FreePages.Last();
				FreePages.RemoveAt(FreePages.Num() - 1, EAllowShrinking::No);
				UsedPages.Add(Page);
				return Page;
			}

			void FreePage(FPage* Page)
			{
				FScopeLock ScopeLock(&CriticalSection);

				int32 Index = UsedPages.Find(Page);
				check(Index >= 0);
				UsedPages.RemoveAt(Index, EAllowShrinking::No);
				FreePages.Add(Page);
			}

			TArray<FPage*, TInlineAllocator<8> > FreePages;
			TArray<FPage*, TInlineAllocator<8> > UsedPages;

			FCriticalSection CriticalSection;

		} GMemoryPagePool;
#endif

		FPage* FPage::AllocatePage(SIZE_T PageSize)
		{
#if USE_PAGE_POOLING
			return GMemoryPagePool.AllocatePage();
#else
			return new FPage(PageSize);
#endif
		}

		void FPage::FreePage(FPage* Page)
		{
#if USE_PAGE_POOLING
			GMemoryPagePool.FreePage(Page);
#else
			delete Page;
#endif
		}
	}
}

// Returns "TEXCOORD" if Semantic is "TEXCOORD4" and OutStartIndex=4; return nullptr if the semantic didn't have a number
static inline TCHAR* GetNonDigitSemanticPrefix(CrossCompiler::FLinearAllocator* Allocator, const TCHAR* Semantic, uint32& OutStartIndex)
{
	const TCHAR* StartOfDigit = Semantic;
	do 
	{
		if (*StartOfDigit >= '0' && *StartOfDigit <= '9')
		{
			break;
		}
		++StartOfDigit;
	}
	while (*StartOfDigit);

	if (!*StartOfDigit)
	{
		return nullptr;
	}

	OutStartIndex = FCString::Atoi(StartOfDigit);
	TCHAR* Prefix = Allocator->Strdup(Semantic);
	Prefix[StartOfDigit - Semantic] = 0;
	return Prefix;
}


static inline TCHAR* MakeIndexedSemantic(CrossCompiler::FLinearAllocator* Allocator, const TCHAR* Semantic, uint32 Index)
{
	check(Semantic);
	FString Out = FString::Printf(TEXT("%s%d"), Semantic, Index);
	return Allocator->Strdup(Out);
}

static bool CheckSimpleVectorType(const TCHAR* SimpleType)
{
	if (!FCString::Strncmp(SimpleType, TEXT("float"), 5))
	{
		SimpleType += 5;
	}
	else if (!FCString::Strncmp(SimpleType, TEXT("int"), 3))
	{
		SimpleType += 3;
	}
	else if (!FCString::Strncmp(SimpleType, TEXT("half"), 4))
	{
		SimpleType += 4;
	}
	else
	{
		return false;
	}
	return FChar::IsDigit(SimpleType[0]) && SimpleType[1] == 0;
}

struct FRemoveAlgorithm
{
	FString EntryPoint;
	bool bSuccess;
	FString GeneratedCode;
	TArray<FString> Errors;
	CrossCompiler::FLinearAllocator* Allocator;
	CrossCompiler::FSourceInfo SourceInfo;

	TArray<FString> RemovedSemantics;

	struct FBodyContext
	{
		TArray<CrossCompiler::AST::FStructSpecifier*> NewStructs;

		// Instructions before calling the original function
		TArray<CrossCompiler::AST::FNode*> PreInstructions;

		// Call to the original function
		CrossCompiler::AST::FFunctionExpression* CallToOriginalFunction;

		// Instructions after calling the original function
		TArray<CrossCompiler::AST::FNode*> PostInstructions;

		// Final instruction
		CrossCompiler::AST::FNode* FinalInstruction;

		// Parameter of the new entry point
		TArray<CrossCompiler::AST::FParameterDeclarator*> NewFunctionParameters;

		FBodyContext() :
			CallToOriginalFunction(nullptr),
			FinalInstruction(nullptr)
		{
		}
	};

	FRemoveAlgorithm() :
		bSuccess(false),
		Allocator(nullptr)
	{
	}

	static CrossCompiler::AST::FExpression* MakeIdentifierExpression(CrossCompiler::FLinearAllocator* Allocator, const CrossCompiler::AST::FIdentifier* Name, const CrossCompiler::FSourceInfo& SourceInfo)
	{
		using namespace CrossCompiler::AST;
		FExpression* Expression = new(Allocator) FExpression(Allocator, EOperators::Identifier, SourceInfo);
		Expression->Identifier = Name;
		return Expression;
	}

	static CrossCompiler::AST::FExpression* MakeIdentifierExpression(CrossCompiler::FLinearAllocator* Allocator, const TCHAR* Name, const CrossCompiler::FSourceInfo& SourceInfo)
	{
		return MakeIdentifierExpression(Allocator, CrossCompiler::AST::FIdentifier::New(Allocator, Name), SourceInfo);
	}

	CrossCompiler::AST::FFunctionDefinition* FindEntryPointAndPopulateSymbolTable(CrossCompiler::TLinearArray<CrossCompiler::AST::FNode*>& ASTNodes, TArray<CrossCompiler::AST::FStructSpecifier*>& OutMiniSymbolTable, FString* OutOptionalWriteNodes)
	{
		using namespace CrossCompiler::AST;
		FFunctionDefinition* EntryFunction = nullptr;
		for (int32 Index = 0; Index < ASTNodes.Num(); ++Index)
		{
			auto* Node = ASTNodes[Index];
			if (FDeclaratorList* DeclaratorList = Node->AsDeclaratorList())
			{
				// Skip unnamed structures
				if (DeclaratorList->Type->Specifier->Structure && DeclaratorList->Type->Specifier->Structure->Name)
				{
					OutMiniSymbolTable.Add(DeclaratorList->Type->Specifier->Structure);
				}
			}
			else if (FFunctionDefinition* FunctionDefinition = Node->AsFunctionDefinition())
			{
				if (FunctionDefinition->Prototype->Identifier->Equals(EntryPoint))
				{
					EntryFunction = FunctionDefinition;
				}
			}

			if (OutOptionalWriteNodes)
			{
				FASTWriter Writer(*OutOptionalWriteNodes);
				Node->Write(Writer);
			}
		}

		return EntryFunction;
	}

	CrossCompiler::AST::FFullySpecifiedType* CloneType(CrossCompiler::AST::FFullySpecifiedType* InType, bool bStripInOut = true)
	{
		auto* New = new(Allocator)CrossCompiler::AST::FFullySpecifiedType(Allocator, SourceInfo);
		New->Qualifier = InType->Qualifier;
		if (bStripInOut)
		{
			New->Qualifier.bIn = false;
			New->Qualifier.bOut = false;
		}
		New->Specifier = InType->Specifier;
		return New;
	}

	CrossCompiler::AST::FStructSpecifier* CreateNewStructSpecifier(const TCHAR* TypeName, TArray<CrossCompiler::AST::FStructSpecifier*>& NewStructs)
	{
		auto* NewReturnType = new(Allocator) CrossCompiler::AST::FStructSpecifier(Allocator, SourceInfo);
		NewReturnType->Name = Allocator->Strdup(TypeName);
		NewStructs.Add(NewReturnType);
		return NewReturnType;
	}

	CrossCompiler::AST::FFunctionDefinition* CreateNewEntryFunction(CrossCompiler::AST::FCompoundStatement* Body, 
		CrossCompiler::AST::FFullySpecifiedType* ReturnType, 
		TArray<CrossCompiler::AST::FParameterDeclarator*>& Parameters, 
		CrossCompiler::TLinearArray<CrossCompiler::AST::FAttribute*>& FunctionAttributes)
	{
		using namespace CrossCompiler::AST;
		// New Entry definition/prototype
		FFunctionDefinition* NewEntryFunction = new(Allocator) FFunctionDefinition(Allocator, SourceInfo);
		NewEntryFunction->Prototype = new(Allocator) FFunction(Allocator, SourceInfo);
		NewEntryFunction->Prototype->Identifier = FIdentifier::New(Allocator, *(EntryPoint + TEXT("__OPTIMIZED")));
		NewEntryFunction->Prototype->ReturnType = ReturnType;
		NewEntryFunction->Prototype->Attributes = FunctionAttributes;
		NewEntryFunction->Body = Body;
		for (auto* Parameter : Parameters)
		{
			NewEntryFunction->Prototype->Parameters.Add(Parameter);
		}

		return NewEntryFunction;
	}

	CrossCompiler::AST::FFullySpecifiedType* MakeSimpleType(const TCHAR* Name)
	{
		auto* ReturnType = new(Allocator) CrossCompiler::AST::FFullySpecifiedType(Allocator, SourceInfo);
		ReturnType->Specifier = new(Allocator) CrossCompiler::AST::FTypeSpecifier(Allocator, SourceInfo);
		ReturnType->Specifier->TypeName = Allocator->Strdup(Name);
		return ReturnType;
	};

	CrossCompiler::AST::FStructSpecifier* FindStructSpecifier(TArray<CrossCompiler::AST::FStructSpecifier*>& MiniSymbolTable, const TCHAR* StructName)
	{
		for (auto* StructSpecifier : MiniSymbolTable)
		{
			if (!FCString::Strcmp(StructSpecifier->Name, StructName))
			{
				return StructSpecifier;
			}
		}

		return nullptr;
	}

	// Case-insensitive when working with Semantics
	static bool IsStringInArray(const TConstArrayView<FStringView> Array, const TCHAR* Semantic)
	{
		for (FStringView String : Array)
		{
			if (String.Equals(Semantic, ESearchCase::IgnoreCase))
			{
				return true;
			}
		}

		return false;
	};

	static bool IsSubstringInArray(const TConstArrayView<FStringView> Array, const TCHAR* Semantic)
	{
		for (FStringView String : Array)
		{
			if (UE::String::FindFirst(String, Semantic, ESearchCase::IgnoreCase) != INDEX_NONE)
			{
				return true;
			}
		}

		return false;
	};

	bool CopyMember(CrossCompiler::AST::FDeclaration* Declaration, const TCHAR* DestPrefix, const TCHAR* SourcePrefix, TArray<CrossCompiler::AST::FNode*>& InstructionList)
	{
		using namespace CrossCompiler::AST;

		// Add copy statement(s)
		FString LHSName = DestPrefix;
		LHSName += '.';
		LHSName += Declaration->Identifier->ToStringView();
		FString RHSName = SourcePrefix;
		RHSName += '.';
		RHSName += Declaration->Identifier->ToStringView();

		if (Declaration->bIsArray)
		{
			uint32 ArrayLength = 0;
			if (!GetArrayLength(Declaration, ArrayLength))
			{
				return false;
			}

			for (uint32 Index = 0; Index < ArrayLength; ++Index)
			{
				FString LHSElement = FString::Printf(TEXT("%s[%d]"), *LHSName, Index);
				FString RHSElement = FString::Printf(TEXT("%s[%d]"), *RHSName, Index);
				auto* LHS = MakeIdentifierExpression(Allocator, *LHSElement, SourceInfo);
				auto* RHS = MakeIdentifierExpression(Allocator, *RHSElement, SourceInfo);
				auto* Assignment = new(Allocator) FBinaryExpression(Allocator, EOperators::Assign, LHS, RHS, SourceInfo);
				InstructionList.Add(new(Allocator) FExpressionStatement(Allocator, Assignment, SourceInfo));
			}
		}
		else
		{
			auto* LHS = MakeIdentifierExpression(Allocator, *LHSName, SourceInfo);
			auto* RHS = MakeIdentifierExpression(Allocator, *RHSName, SourceInfo);
			auto* Assignment = new(Allocator) FBinaryExpression(Allocator, EOperators::Assign, LHS, RHS, SourceInfo);
			InstructionList.Add(new(Allocator) FExpressionStatement(Allocator, Assignment, SourceInfo));
		}

		return true;
	}

	CrossCompiler::AST::FDeclaratorList* CreateLocalVariable(const TCHAR* Type, CrossCompiler::AST::FIdentifier* VariableName, CrossCompiler::AST::FExpression* Initializer = nullptr)
	{
		using namespace CrossCompiler::AST;
		auto* LocalVarDeclaratorList = new(Allocator) FDeclaratorList(Allocator, SourceInfo);
		LocalVarDeclaratorList->Type = MakeSimpleType(Type);
		auto* LocalVarDeclaration = new(Allocator) FDeclaration(Allocator, SourceInfo);
		LocalVarDeclaration->Identifier = VariableName;
		LocalVarDeclaration->Initializer = Initializer;
		LocalVarDeclaratorList->Declarations.Add(LocalVarDeclaration);
		return LocalVarDeclaratorList;
	}

	CrossCompiler::AST::FDeclaratorList* CreateLocalVariable( const TCHAR* Type, FStringView VariableName, CrossCompiler::AST::FExpression* Initializer = nullptr)
	{
		return CreateLocalVariable(Type, CrossCompiler::AST::FIdentifier::New(Allocator, VariableName), Initializer);
	}

	CrossCompiler::AST::FCompoundStatement* AddStatementsToBody(FBodyContext& Return, CrossCompiler::AST::FNode* CallInstruction)
	{
		CrossCompiler::AST::FCompoundStatement* Body = new(Allocator)CrossCompiler::AST::FCompoundStatement(Allocator, SourceInfo);
		for (auto* Instruction : Return.PreInstructions)
		{
			Body->Statements.Add(Instruction);
		}

		if (CallInstruction)
		{
			Body->Statements.Add(CallInstruction);
		}

		for (auto* Instruction : Return.PostInstructions)
		{
			Body->Statements.Add(Instruction);
		}

		if (Return.FinalInstruction)
		{
			Body->Statements.Add(Return.FinalInstruction);
		}

		return Body;
	}


	bool GetArrayLength(CrossCompiler::AST::FDeclaration* A, uint32& OutLength)
	{
		using namespace CrossCompiler::AST;
		if (!A->bIsArray)
		{
			Errors.Add(FString::Printf(TEXT("RemoveUnusedOutputs: %s is expected to be an array!"), A->Identifier->GetData()));
			return false;
		}
		else
		{
			if (A->ArraySize.Num() > 1)
			{
				Errors.Add(FString::Printf(TEXT("RemoveUnusedOutputs: No support for multidimensional arrays on %s!"), A->Identifier->GetData()));
				return false;
			}

			for (int32 Index = 0; Index < A->ArraySize.Num(); ++Index)
			{
				int32 DimA = 0;
				if (!A->ArraySize[Index]->GetConstantIntValue(DimA))
				{
					Errors.Add(FString::Printf(TEXT("RemoveUnusedOutputs: Array %s is not a compile-time constant expression!"), A->Identifier->GetData()));
					return false;
				}

				OutLength = DimA;
			}
		}

		return true;
	}
};

struct FRemoveUnusedOutputs : FRemoveAlgorithm
{
	const TConstArrayView<FStringView> UsedOutputs;
	const TConstArrayView<FStringView> Exceptions;

	struct FOutputsBodyContext : FBodyContext
	{
		CrossCompiler::AST::FStructSpecifier* NewReturnStruct;

		// Expression (might be assignment) calling CallToOriginalFunction
		CrossCompiler::AST::FExpression* CallExpression;

		const TCHAR* ReturnVariableName;
		const TCHAR* ReturnTypeName;

		// Parameter of the new entry point
		TArray<CrossCompiler::AST::FParameterDeclarator*> NewFunctionParameters;

		FOutputsBodyContext() :
			NewReturnStruct(nullptr),
			CallExpression(nullptr),
			ReturnVariableName(TEXT("OptimizedReturn")),
			ReturnTypeName(TEXT("FOptimizedReturn"))
		{
		}
	};

	FRemoveUnusedOutputs(const TConstArrayView<FStringView> InUsedOutputs, const TConstArrayView<FStringView> InExceptions) :
		UsedOutputs(InUsedOutputs),
		Exceptions(InExceptions)
	{
	}

	bool SetupReturnType(CrossCompiler::AST::FFunctionDefinition* EntryFunction, TArray<CrossCompiler::AST::FStructSpecifier*>& MiniSymbolTable, FOutputsBodyContext& OutReturn)
	{
		using namespace CrossCompiler::AST;

		// Create the new return type, local variable and the final return statement
		{
			// New return type
			OutReturn.NewReturnStruct = CreateNewStructSpecifier(OutReturn.ReturnTypeName, OutReturn.NewStructs);

			// Local Variable
			OutReturn.PreInstructions.Add(CreateLocalVariable(OutReturn.NewReturnStruct->Name, OutReturn.ReturnVariableName));

			// Return Statement
			auto* ReturnStatement = new(Allocator) FJumpStatement(Allocator, EJumpType::Return, SourceInfo);
			ReturnStatement->OptionalExpression = MakeIdentifierExpression(Allocator, OutReturn.ReturnVariableName, SourceInfo);
			OutReturn.FinalInstruction = ReturnStatement;
		}

		auto* ReturnType = EntryFunction->Prototype->ReturnType;
		if (ReturnType && ReturnType->Specifier && ReturnType->Specifier->TypeName)
		{
			const TCHAR* ReturnTypeName = ReturnType->Specifier->TypeName;
			if (!EntryFunction->Prototype->ReturnSemantic && !FCString::Strcmp(ReturnTypeName, TEXT("void")))
			{
				return true;
			}
			else
			{
				// Confirm this is a struct living in the symbol table
				FStructSpecifier* OriginalStructSpecifier = FindStructSpecifier(MiniSymbolTable, ReturnTypeName);
				if (OriginalStructSpecifier)
				{
					return ProcessStructReturnType(OriginalStructSpecifier, MiniSymbolTable, OutReturn);
				}
				else if (CheckSimpleVectorType(ReturnTypeName))
				{
					if (EntryFunction->Prototype->ReturnSemantic)
					{
						ProcessSimpleReturnType(ReturnTypeName, EntryFunction->Prototype->ReturnSemantic ? EntryFunction->Prototype->ReturnSemantic->Semantic : nullptr, OutReturn);
						return true;
					}
					else
					{
						Errors.Add(FString::Printf(TEXT("RemoveUnusedOutputs: Function %s with return type %s doesn't have a return semantic"), *EntryPoint, ReturnTypeName));
					}
				}
				else
				{
					Errors.Add(FString::Printf(TEXT("RemoveUnusedOutputs: Invalid return type %s for function %s"), ReturnTypeName, *EntryPoint));
				}
			}
		}
		else
		{
			Errors.Add(FString::Printf(TEXT("RemoveUnusedOutputs: Internal error trying to determine return type")));
		}

		return false;
	};

	void RemoveUnusedOutputs(CrossCompiler::TLinearArray<CrossCompiler::AST::FNode*>& ASTNodes)
	{
		using namespace CrossCompiler::AST;

		// Find Entry point from original AST nodes
		TArray<FStructSpecifier*> MiniSymbolTable;
		FString Test;
		FFunctionDefinition* EntryFunction = FindEntryPointAndPopulateSymbolTable(ASTNodes, MiniSymbolTable, &Test);
		if (!EntryFunction)
		{
			Errors.Add(FString::Printf(TEXT("RemoveUnusedOutputs: Unable to find entry point %s"), *EntryPoint));
			bSuccess = false;
			return;
		}
		//FPlatformMisc::LowLevelOutputDebugString(*Test);

		SourceInfo = EntryFunction->SourceInfo;

		FOutputsBodyContext BodyContext;

		// Setup the call to the original entry point
		BodyContext.CallToOriginalFunction = new(Allocator) FFunctionExpression(Allocator, SourceInfo, MakeIdentifierExpression(Allocator, *EntryPoint, SourceInfo));

		if (!SetupReturnType(EntryFunction, MiniSymbolTable, BodyContext))
		{
			bSuccess = false;
			return;
		}

		if (!ProcessOriginalParameters(EntryFunction, MiniSymbolTable, BodyContext))
		{
			bSuccess = false;
			return;
		}

		// Real call statement
		if (BodyContext.CallToOriginalFunction && !BodyContext.CallExpression)
		{
			BodyContext.CallExpression = BodyContext.CallToOriginalFunction;
		}
		auto* CallInstruction = new(Allocator) CrossCompiler::AST::FExpressionStatement(Allocator, BodyContext.CallExpression, SourceInfo);

		FCompoundStatement* Body = AddStatementsToBody(BodyContext, CallInstruction);
		FFunctionDefinition* NewEntryFunction = CreateNewEntryFunction(Body, MakeSimpleType(BodyContext.NewReturnStruct->Name), BodyContext.NewFunctionParameters, EntryFunction->Prototype->Attributes);
		EntryPoint = NewEntryFunction->Prototype->Identifier->ToString();
		WriteGeneratedOutCode(NewEntryFunction, BodyContext.NewStructs, GeneratedCode);
		bSuccess = true;
	}

	void WriteGeneratedOutCode(CrossCompiler::AST::FFunctionDefinition* NewEntryFunction, TArray<CrossCompiler::AST::FStructSpecifier*>& NewStructs, FString& OutGeneratedCode)
	{
		CrossCompiler::AST::FASTWriter Writer(OutGeneratedCode);
		GeneratedCode = TEXT("#line 1 \"RemoveUnusedOutputs.usf\"\n// Generated Entry Point: ");
		GeneratedCode += NewEntryFunction->Prototype->Identifier->ToStringView();
		GeneratedCode += TEXT("\n");
		if (UsedOutputs.Num() > 0)
		{
			GeneratedCode += TEXT("// Requested UsedOutputs:");
			for (int32 Index = 0; Index < UsedOutputs.Num(); ++Index)
			{
				GeneratedCode += (Index == 0) ? TEXT(" ") : TEXT(", ");
				GeneratedCode += UsedOutputs[Index];
			}
			GeneratedCode += TEXT("\n");
		}
		if (RemovedSemantics.Num() > 0)
		{
			GeneratedCode += TEXT("// Removed Outputs:");
			for (int32 Index = 0; Index < RemovedSemantics.Num(); ++Index)
			{
				GeneratedCode += (Index == 0) ? TEXT(" ") : TEXT(", ");
				GeneratedCode += RemovedSemantics[Index];
			}
			GeneratedCode += TEXT("\n");
		}
		for (auto* Struct : NewStructs)
		{
			auto* Declarator = new(Allocator) CrossCompiler::AST::FDeclaratorList(Allocator, SourceInfo);
			Declarator->Declarations.Add(Struct);
			Declarator->Write(Writer);
		}
		NewEntryFunction->Write(Writer);
		//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("*********************************\n%s\n"), *GeneratedCode);
	}

	void ProcessSimpleOutParameter(CrossCompiler::AST::FParameterDeclarator* ParameterDeclarator, FOutputsBodyContext& BodyContext)
	{
		using namespace CrossCompiler::AST;

		// Only add the parameter if it needs to also be returned
		bool bRequiresToBeInReturnStruct = IsSemanticUsed(ParameterDeclarator->Semantic);

		if (bRequiresToBeInReturnStruct)
		{
			// Add the member to the return struct
			auto* MemberDeclaratorList = new(Allocator) FDeclaratorList(Allocator, SourceInfo);
			MemberDeclaratorList->Type = CloneType(ParameterDeclarator->Type);
			auto* MemberDeclaration = new(Allocator) FDeclaration(Allocator, SourceInfo);
			MemberDeclaration->Identifier = ParameterDeclarator->Identifier;
			MemberDeclaration->Semantic = ParameterDeclarator->Semantic;
			MemberDeclaratorList->Declarations.Add(MemberDeclaration);

			// Add it to the return struct type
			check(BodyContext.NewReturnStruct);
			BodyContext.NewReturnStruct->Members.Add(MemberDeclaratorList);

			// Add the parameter to the actual function call
			FString ParameterName = BodyContext.ReturnVariableName;
			ParameterName += TEXT(".");
			ParameterName += ParameterDeclarator->Identifier->ToStringView();
			auto* Parameter = MakeIdentifierExpression(Allocator, *ParameterName, SourceInfo);
			BodyContext.CallToOriginalFunction->Expressions.Add(Parameter);
		}
		else
		{
			// Make a local to receive the out parameter
			auto* LocalVar = CreateLocalVariable(ParameterDeclarator->Type->Specifier->TypeName, ParameterDeclarator->Identifier);
			BodyContext.PreInstructions.Add(LocalVar);

			// Add the parameter to the actual function call
			auto* Parameter = MakeIdentifierExpression(Allocator, ParameterDeclarator->Identifier, SourceInfo);
			BodyContext.CallToOriginalFunction->Expressions.Add(Parameter);
		}
	}

	void ProcessSimpleReturnType(const TCHAR* TypeName, const TCHAR* Semantic, FOutputsBodyContext& BodyContext)
	{
		using namespace CrossCompiler::AST;

		// Create a member to return this simple type out
		auto* MemberDeclaratorList = new(Allocator) FDeclaratorList(Allocator, SourceInfo);
		MemberDeclaratorList->Type = MakeSimpleType(TypeName);
		auto* MemberDeclaration = new(Allocator) FDeclaration(Allocator, SourceInfo);
		MemberDeclaration->Identifier = FIdentifier::New(Allocator, TEXTVIEW("SimpleReturn"));
		MemberDeclaration->Semantic = new(Allocator) FSemanticSpecifier(Allocator, Semantic, SourceInfo);
		MemberDeclaratorList->Declarations.Add(MemberDeclaration);

		// Add it to the return struct type
		check(BodyContext.NewReturnStruct);
		BodyContext.NewReturnStruct->Members.Add(MemberDeclaratorList);

		// Create the LHS of the member assignment
		FString MemberName = BodyContext.ReturnVariableName;
		MemberName += TEXT(".");
		MemberName += MemberDeclaration->Identifier->ToStringView();
		auto* SimpleTypeMember = MakeIdentifierExpression(Allocator, *MemberName, SourceInfo);

		// Create an assignment from the call the original function
		check(BodyContext.CallToOriginalFunction);
		BodyContext.CallExpression = new(Allocator) FBinaryExpression(Allocator, EOperators::Assign, SimpleTypeMember, BodyContext.CallToOriginalFunction, SourceInfo);
	}

	bool ProcessStructReturnType(CrossCompiler::AST::FStructSpecifier* StructSpecifier, TArray<CrossCompiler::AST::FStructSpecifier*>& MiniSymbolTable, FOutputsBodyContext& BodyContext)
	{
		using namespace CrossCompiler::AST;

		// Add a local variable to receive the output from the function
		FString LocalStructVarName = TEXT("Local_");
		LocalStructVarName += StructSpecifier->Name;
		auto* LocalStructVariable = CreateLocalVariable(StructSpecifier->Name, *LocalStructVarName);
		BodyContext.PreInstructions.Add(LocalStructVariable);

		// Create the LHS of the member assignment
		auto* SimpleTypeMember = MakeIdentifierExpression(Allocator, *LocalStructVarName, SourceInfo);

		// Create an assignment from the call the original function
		check(BodyContext.CallToOriginalFunction);
		BodyContext.CallExpression = new(Allocator) FBinaryExpression(Allocator, EOperators::Assign, SimpleTypeMember, BodyContext.CallToOriginalFunction, SourceInfo);

		// Add all the members and the copies to the return struct
		return AddUsedOutputMembers(BodyContext.NewReturnStruct, BodyContext.ReturnVariableName, StructSpecifier, *LocalStructVarName, MiniSymbolTable, BodyContext);
	}

	bool ProcessStructOutParameter(CrossCompiler::AST::FParameterDeclarator* ParameterDeclarator, CrossCompiler::AST::FStructSpecifier* OriginalStructSpecifier, TArray<CrossCompiler::AST::FStructSpecifier*>& MiniSymbolTable, FOutputsBodyContext& BodyContext)
	{
		// Add a local variable to receive the output from the function
		FString LocalStructVarName = TEXT("Local_");
		LocalStructVarName += OriginalStructSpecifier->Name;
		LocalStructVarName += TEXT("_OUT");
		auto* LocalStructVariable = CreateLocalVariable(OriginalStructSpecifier->Name, *LocalStructVarName);
		BodyContext.PreInstructions.Add(LocalStructVariable);

		// Add the parameter to the actual function call
		auto* Parameter = MakeIdentifierExpression(Allocator, *LocalStructVarName, SourceInfo);
		BodyContext.CallToOriginalFunction->Expressions.Add(Parameter);

		return AddUsedOutputMembers(BodyContext.NewReturnStruct, BodyContext.ReturnVariableName, OriginalStructSpecifier, *LocalStructVarName, MiniSymbolTable, BodyContext);
	}

	bool ProcessOriginalParameters(CrossCompiler::AST::FFunctionDefinition* EntryFunction, TArray<CrossCompiler::AST::FStructSpecifier*>& MiniSymbolTable, FOutputsBodyContext& BodyContext)
	{
		using namespace CrossCompiler::AST;
		for (FNode* ParamNode : EntryFunction->Prototype->Parameters)
		{
			FParameterDeclarator* ParameterDeclarator = ParamNode->AsParameterDeclarator();
			check(ParameterDeclarator);

			if (ParameterDeclarator->Type->Qualifier.bOut)
			{
				if (ParameterDeclarator->Semantic)
				{
					ProcessSimpleOutParameter(ParameterDeclarator, BodyContext);
				}
				else
				{
					// Confirm this is a struct living in the symbol table
					FStructSpecifier* OriginalStructSpecifier = FindStructSpecifier(MiniSymbolTable, ParameterDeclarator->Type->Specifier->TypeName);
					if (OriginalStructSpecifier)
					{
						if (!ProcessStructOutParameter(ParameterDeclarator, OriginalStructSpecifier, MiniSymbolTable, BodyContext))
						{
							return false;
						}
					}
					else if (CheckSimpleVectorType(ParameterDeclarator->Type->Specifier->TypeName))
					{
						Errors.Add(FString::Printf(TEXT("RemoveUnusedOutputs: Function %s with out parameter %s doesn't have a return semantic"), *EntryPoint, ParameterDeclarator->Identifier->GetData()));
						return false;
					}
					else
					{
						Errors.Add(FString::Printf(TEXT("RemoveUnusedOutputs: Invalid return type %s for out parameter %s for function %s"), ParameterDeclarator->Type->Specifier->TypeName, ParameterDeclarator->Identifier->GetData(), *EntryPoint));
						return false;
					}
				}
			}
			else
			{
				// Add this parameter as an input to the new function
				BodyContext.NewFunctionParameters.Add(ParameterDeclarator);

				// Add the parameter to the actual function call
				auto* Parameter = MakeIdentifierExpression(Allocator, ParameterDeclarator->Identifier, SourceInfo);
				BodyContext.CallToOriginalFunction->Expressions.Add(Parameter);
			}
		}

		return true;
	}

	// Returns true if the semantic names is explicitly exempt from removal by the shader backend. This applies to output-only system values such as SV_ClipDistance.
	bool IsSemanticExempt(const TCHAR* SemanticName) const
	{
		return IsSubstringInArray(Exceptions, SemanticName);
	}

	bool IsSemanticExempt(const CrossCompiler::AST::FSemanticSpecifier* Semantic) const
	{
		return Semantic && IsSemanticExempt(Semantic->Semantic);
	}

	bool IsSemanticUsed(const TCHAR* SemanticName) const
	{
		if (bLeaveAllUsed || IsStringInArray(UsedOutputs, SemanticName) || IsSemanticExempt(SemanticName))
		{
			return true;
		}

		// Try the centroid modifier for safety
		if (!FCString::Stristr(SemanticName, TEXT("_centroid")))
		{
			FString Centroid = SemanticName;
			Centroid += "_centroid";
			return IsStringInArray(UsedOutputs, SemanticName);
		}

		return false;
	}

	bool IsSemanticUsed(const CrossCompiler::AST::FSemanticSpecifier* Semantic) const
	{
		return Semantic ? IsSemanticUsed(Semantic->Semantic) : false;
	}

	bool AddUsedOutputMembers(CrossCompiler::AST::FStructSpecifier* DestStruct, const TCHAR* DestPrefix, CrossCompiler::AST::FStructSpecifier* SourceStruct, const TCHAR* SourcePrefix, TArray<CrossCompiler::AST::FStructSpecifier*>& MiniSymbolTable, FBodyContext& BodyContext)
	{
		using namespace CrossCompiler::AST;

		for (auto* Member : SourceStruct->Members)
		{
			FDeclaratorList* MemberDeclarator = Member->AsDeclaratorList();
			if (MemberDeclarator)
			{
				for (auto* DeclarationNode : MemberDeclarator->Declarations)
				{
					FDeclaration* MemberDeclaration = DeclarationNode->AsDeclaration();
					check(MemberDeclaration);
					if (MemberDeclaration->Semantic)
					{
						if (MemberDeclaration->bIsArray)
						{
							uint32 ArrayLength = 0;
							if (!GetArrayLength(MemberDeclaration, ArrayLength))
							{
								return false;
							}

							uint32 StartIndex = 0;
							TCHAR* ElementSemanticPrefix = GetNonDigitSemanticPrefix(Allocator, MemberDeclaration->Semantic->Semantic, StartIndex);
							if (!ElementSemanticPrefix)
							{
								Errors.Add(FString::Printf(TEXT("RemoveUnusedOutputs: Member (%s) %s : %s is expected to have an indexed semantic!"), MemberDeclarator->Type->Specifier->TypeName, MemberDeclaration->Identifier->GetData(), MemberDeclaration->Semantic->Semantic));

								// Fatal: Array of non-indexed semantic (eg float4 Colors[4] : MYSEMANTIC; )
								// Assume semantic is used and just fallback
								auto* NewDeclaratorList = new(Allocator) FDeclaratorList(Allocator, MemberDeclarator->SourceInfo);
								NewDeclaratorList->Type = CloneType(MemberDeclarator->Type);
								NewDeclaratorList->Declarations.Add(MemberDeclaration);
								DestStruct->Members.Add(NewDeclaratorList);

								CopyMember(MemberDeclaration, DestPrefix, SourcePrefix, BodyContext.PostInstructions);
							}
							else
							{
								for (uint32 Index = 0; Index < ArrayLength; ++Index)
								{
									TCHAR* ElementSemantic = MakeIndexedSemantic(Allocator, ElementSemanticPrefix, StartIndex + Index);
									if (IsSemanticUsed(ElementSemantic))
									{
										auto* NewMemberDeclaration = new(Allocator) FDeclaration(Allocator, MemberDeclaration->SourceInfo);
										NewMemberDeclaration->Semantic = new(Allocator) FSemanticSpecifier(Allocator, ElementSemantic, MemberDeclaration->SourceInfo);
										NewMemberDeclaration->Identifier = FIdentifier::New(Allocator, FString::Printf(TEXT("%s_%d"), MemberDeclaration->Identifier->GetData(), Index));

										// Add member to struct
										auto* NewDeclaratorList = new(Allocator) FDeclaratorList(Allocator, MemberDeclarator->SourceInfo);
										NewDeclaratorList->Type = CloneType(MemberDeclarator->Type);
										NewDeclaratorList->Declarations.Add(NewMemberDeclaration);
										DestStruct->Members.Add(NewDeclaratorList);

										FString LHSElement = FString::Printf(TEXT("%s.%s"), DestPrefix, NewMemberDeclaration->Identifier->GetData());
										FString RHSElement = FString::Printf(TEXT("%s.%s[%d]"), SourcePrefix, MemberDeclaration->Identifier->GetData(), Index);

										auto* LHS = MakeIdentifierExpression(Allocator, *LHSElement, SourceInfo);
										auto* RHS = MakeIdentifierExpression(Allocator, *RHSElement, SourceInfo);
										auto* Assignment = new(Allocator) FBinaryExpression(Allocator, EOperators::Assign, LHS, RHS, SourceInfo);
										BodyContext.PostInstructions.Add(new(Allocator) FExpressionStatement(Allocator, Assignment, SourceInfo));
									}
									else
									{
										RemovedSemantics.Add(ElementSemantic);
									}
								}
							}
						}
						else if (IsSemanticUsed(MemberDeclaration->Semantic))
						{
							// Add member to struct
							auto* NewDeclaratorList = new(Allocator) FDeclaratorList(Allocator, MemberDeclarator->SourceInfo);
							NewDeclaratorList->Type = CloneType(MemberDeclarator->Type);
							NewDeclaratorList->Declarations.Add(MemberDeclaration);
							DestStruct->Members.Add(NewDeclaratorList);

							CopyMember(MemberDeclaration, DestPrefix, SourcePrefix, BodyContext.PostInstructions);
						}
						else
						{
							RemovedSemantics.Add(MemberDeclaration->Semantic->Semantic);
						}
					}
					else
					{
						if (!MemberDeclarator->Type || !MemberDeclarator->Type->Specifier || !MemberDeclarator->Type->Specifier->TypeName)
						{
							Errors.Add(FString::Printf(TEXT("RemoveUnusedOutputs: Internal error tracking down nested type %s"), MemberDeclaration->Identifier->GetData()));
							return false;
						}

						// No semantic, so make sure this is a nested struct, or error that it's missing a semantic
						FStructSpecifier* NestedStructSpecifier = FindStructSpecifier(MiniSymbolTable, MemberDeclarator->Type->Specifier->TypeName);
						if (!NestedStructSpecifier)
						{
							Errors.Add(FString::Printf(TEXT("RemoveUnusedOutputs: Member (%s) %s is expected to have a semantic!"), MemberDeclarator->Type->Specifier->TypeName, MemberDeclaration->Identifier->GetData()));
							return false;
						}

						// Add all the elements of this new struct into the return type
						FString NewSourcePrefix = SourcePrefix;
						NewSourcePrefix += TEXT(".");
						NewSourcePrefix += MemberDeclaration->Identifier->ToStringView();
						AddUsedOutputMembers(DestStruct, DestPrefix, NestedStructSpecifier, *NewSourcePrefix, MiniSymbolTable, BodyContext);
					}
				}
			}
			else
			{
				// Clone member function to struct
				DestStruct->Members.Add(Member);
				check(0);
/*
				auto* NewDeclaratorList = new(Allocator) FDeclaratorList(Allocator, MemberDeclarator->SourceInfo);
				NewDeclaratorList->Type = CloneType(MemberDeclarator->Type);
				NewDeclaratorList->Declarations.Add(MemberDeclaration);
				DestStruct->Declarations.Add(NewDeclaratorList);
*/
			}
		}

		/*
		Move output only system values to the end, to ensure they don't occupy an output register that might be used by an input register in the pixel shader stage.
		The following output struct for example will mismatch with input interpolators:

			struct FOptimizedReturn
			{
				float4 Position : SV_POSITION;
				float ClipDistance : SV_CLIPDISTANCE; // <-- Only available in vertex output
				nointerpolation uint SomeIndex : SOMEINDEX;
			};

		The pixel shader will end up with the following input struct, because SV_CLIPDISTANCE is not supported in the pixel shader stage:

			struct FOptimizedReturn
			{
				float4 Position : SV_POSITION;
				nointerpolation uint SomeIndex : SOMEINDEX;
			};

		Now SV_CLIPDISTANCE will occupy output register 1 but this register is expected for "SOMEINDEX", resulting in a mismatch and PSO creation failure.
		*/

		auto IsMemberWithExemptSemantic = [this](FNode& Member) -> bool
			{
				CrossCompiler::AST::FDeclaratorList* MemberDeclList = Member.AsDeclaratorList();
				checkf(MemberDeclList->Declarations.Num() == 1, TEXT("Destination structure for removed unused output interpolators must have exactly 1 member in its declaration list per statement, but %d are specified"), MemberDeclList->Declarations.Num());
				CrossCompiler::AST::FDeclaration* MemberDecl = MemberDeclList->Declarations[0]->AsDeclaration();
				return IsSemanticExempt(MemberDecl->Semantic);
			};

		DestStruct->Members.StableSort(
			[this, &IsMemberWithExemptSemantic](CrossCompiler::AST::FNode& LhsNode, CrossCompiler::AST::FNode& RhsNode) -> bool
			{
				return IsMemberWithExemptSemantic(LhsNode) < IsMemberWithExemptSemantic(RhsNode);
			}
		);

		return true;
	}
};

namespace UE::HlslParser
{

bool RemoveUnusedOutputs(FString& InOutSourceCode,
	TConstArrayView<FStringView> InUsedOutputs,
	TConstArrayView<FStringView> InExceptions,
	TConstArrayView<FScopedDeclarations> InScopedDeclarations,
	FString& EntryPoint,
	TArray<FString>& OutErrors)
{
	FString DummyFilename(TEXT("/Engine/Private/RemoveUnusedOutputs.usf"));
	FRemoveUnusedOutputs Data(InUsedOutputs, InExceptions);
	Data.EntryPoint = EntryPoint;
	auto Lambda = [&Data](CrossCompiler::FLinearAllocator* Allocator, CrossCompiler::TLinearArray<CrossCompiler::AST::FNode*>& ASTNodes)
	{
		Data.Allocator = Allocator;
		Data.RemoveUnusedOutputs(ASTNodes);
	};
	CrossCompiler::FCompilerMessages Messages;
	if (!CrossCompiler::Parser::Parse(InOutSourceCode, DummyFilename, Messages, InScopedDeclarations, Lambda))
	{
		Data.Errors.Add(FString(TEXT("RemoveUnusedOutputs: Failed to compile!")));
		OutErrors = Data.Errors;
		for (auto& Message : Messages.MessageList)
		{
			OutErrors.Add(Message.Message);
		}
		return false;
	}

	for (auto& Message : Messages.MessageList)
	{
		OutErrors.Add(Message.Message);
	}

	if (Data.bSuccess)
	{
		InOutSourceCode += (TCHAR)'\n';
		InOutSourceCode += Data.GeneratedCode;
		EntryPoint = Data.EntryPoint;

		return true;
	}

	OutErrors = Data.Errors;
	return false;
}

bool RemoveUnusedOutputs(FString& InOutSourceCode, const TArray<FString>& InUsedOutputs, const TArray<FString>& InExceptions, FString& EntryPoint, TArray<FString>& OutErrors)
{
	const TArray<FStringView> UsedOutputs(MakeArrayView(InUsedOutputs));
	const TArray<FStringView> Exceptions(MakeArrayView(InExceptions));
	return RemoveUnusedOutputs(InOutSourceCode, UsedOutputs, Exceptions, {}, EntryPoint, OutErrors);
}

} // namespace CrossCompiler

struct FRemoveUnusedInputs : FRemoveAlgorithm
{
	const TConstArrayView<FStringView> UsedInputs;

	struct FInputsBodyContext : FBodyContext
	{
		CrossCompiler::AST::FStructSpecifier* NewInputStruct;

		const TCHAR* InputVariableName;
		const TCHAR* InputTypeName;

		FInputsBodyContext() :
			NewInputStruct(nullptr),
			InputVariableName(TEXT("OptimizedInput")),
			InputTypeName(TEXT("FOptimizedInput"))
		{
		}
	};

	FRemoveUnusedInputs(const TConstArrayView<FStringView> InUsedInputs) :
		UsedInputs(InUsedInputs)
	{
	}

	void RemoveUnusedInputs(CrossCompiler::TLinearArray<CrossCompiler::AST::FNode*>& ASTNodes)
	{
		using namespace CrossCompiler::AST;

		// Find Entry point from original AST nodes
		TArray<FStructSpecifier*> MiniSymbolTable;
		FString Test;
		FFunctionDefinition* EntryFunction = FindEntryPointAndPopulateSymbolTable(ASTNodes, MiniSymbolTable, &Test);
		if (!EntryFunction)
		{
			Errors.Add(FString::Printf(TEXT("RemoveUnused: Unable to find entry point %s"), *EntryPoint));
			bSuccess = false;
			return;
		}

		SourceInfo = EntryFunction->SourceInfo;

		FInputsBodyContext BodyContext;

		BodyContext.NewInputStruct = CreateNewStructSpecifier(BodyContext.InputTypeName, BodyContext.NewStructs);

		if (!ProcessOriginalParameters(EntryFunction, MiniSymbolTable, BodyContext))
		{
			bSuccess = false;
			return;
		}

		// Simply pre-inline the original function body.
		// This is cheaper for the down-stream shader compilers as their own inlining passes are expensive when dealing with large functions.
		if (EntryFunction->Body)
		{
			BodyContext.PostInstructions.Append(EntryFunction->Body->Statements);
		}

		auto* Body = AddStatementsToBody(BodyContext, nullptr /*CallInstruction*/);

		if (BodyContext.NewInputStruct->Members.Num() > 0)
		{
			// If the input struct is not empty, add this as an argument to the new entry function
			FParameterDeclarator* Declarator = new(Allocator) FParameterDeclarator(Allocator, SourceInfo);
			Declarator->Type = MakeSimpleType(BodyContext.InputTypeName);
			Declarator->Identifier = FIdentifier::New(Allocator, BodyContext.InputVariableName);
			BodyContext.NewFunctionParameters.Add(Declarator);
		}

		FFunctionDefinition* NewEntryFunction = CreateNewEntryFunction(Body, EntryFunction->Prototype->ReturnType, BodyContext.NewFunctionParameters, EntryFunction->Prototype->Attributes);
		NewEntryFunction->Prototype->ReturnSemantic = EntryFunction->Prototype->ReturnSemantic;

		WriteGeneratedInCode(NewEntryFunction, BodyContext.NewStructs, GeneratedCode);

		EntryPoint = NewEntryFunction->Prototype->Identifier->ToString();
		bSuccess = true;
	}

	bool ProcessOriginalParameters(CrossCompiler::AST::FFunctionDefinition* EntryFunction, TArray<CrossCompiler::AST::FStructSpecifier*>& MiniSymbolTable, FInputsBodyContext& BodyContext)
	{
		using namespace CrossCompiler::AST;
		for (FNode* ParamNode : EntryFunction->Prototype->Parameters)
		{
			FParameterDeclarator* ParameterDeclarator = ParamNode->AsParameterDeclarator();
			check(ParameterDeclarator);
			if (!ParameterDeclarator->Type->Qualifier.bOut)
			{
				if (ParameterDeclarator->Semantic)
				{
					ProcessSimpleInParameter(ParameterDeclarator, BodyContext);
				}
				else
				{
					// Confirm this is a struct living in the symbol table
					FStructSpecifier* OriginalStructSpecifier = FindStructSpecifier(MiniSymbolTable, ParameterDeclarator->Type->Specifier->TypeName);
					if (OriginalStructSpecifier)
					{
						if (!ProcessStructInParameter(ParameterDeclarator, OriginalStructSpecifier, MiniSymbolTable, BodyContext))
						{
							return false;
						}
					}
					else if (CheckSimpleVectorType(ParameterDeclarator->Type->Specifier->TypeName))
					{
						Errors.Add(FString::Printf(TEXT("RemoveUnusedInputs: Function %s with in parameter %s doesn't have a return semantic"), *EntryPoint, ParameterDeclarator->Identifier->GetData()));
						return false;
					}
					else
					{
						Errors.Add(FString::Printf(TEXT("RemoveUnusedInputs: Invalid return type %s for in parameter %s for function %s"), ParameterDeclarator->Type->Specifier->TypeName, ParameterDeclarator->Identifier->GetData(), *EntryPoint));
						return false;
					}
				}
			}
			else
			{
				// Add this parameter as an input to the new function
				BodyContext.NewFunctionParameters.Add(ParameterDeclarator);
			}
		}
		return true;
	}

	bool ProcessStructInParameter(CrossCompiler::AST::FParameterDeclarator* ParameterDeclarator, CrossCompiler::AST::FStructSpecifier* OriginalStructSpecifier, TArray<CrossCompiler::AST::FStructSpecifier*>& MiniSymbolTable, FInputsBodyContext& BodyContext)
	{
		using namespace CrossCompiler::AST;

		auto* Zero = new(Allocator) FExpression(Allocator, EOperators::Literal, SourceInfo);
		Zero->LiteralType = CrossCompiler::ELiteralType::Integer;
		Zero->Identifier = FIdentifier::New(Allocator,TEXTVIEW("0"));
		auto* Initializer = new(Allocator) FUnaryExpression(Allocator, EOperators::TypeCast, Zero, SourceInfo);
		Initializer->TypeSpecifier = MakeSimpleType(OriginalStructSpecifier->Name)->Specifier;

		// Add a local variable to receive the output from the function.
		// The name is simply the parameter name in the real entry point, since it will be inlined directly in the optimized version.

		auto* LocalStructVariable = CreateLocalVariable(OriginalStructSpecifier->Name, ParameterDeclarator->Identifier, Initializer);
		BodyContext.PreInstructions.Add(LocalStructVariable);

		return AddUsedInputMembers(BodyContext.NewInputStruct, BodyContext.InputVariableName, OriginalStructSpecifier, ParameterDeclarator->Identifier->GetData(), MiniSymbolTable, BodyContext);
	}

	bool IsSemanticUsed(const TCHAR* SemanticName)
	{
		if (bLeaveAllUsed || IsStringInArray(UsedInputs, SemanticName))
		{
			return true;
		}

		// Try the centroid modifier for safety
		if (!FCString::Stristr(SemanticName, TEXT("_centroid")))
		{
			FString Centroid = SemanticName;
			Centroid += "_centroid";
			return IsStringInArray(UsedInputs, SemanticName);
		}

		return false;
	}

	bool IsSemanticUsed(const CrossCompiler::AST::FSemanticSpecifier* Semantic)
	{
		return Semantic ? IsSemanticUsed(Semantic->Semantic) : false;
	}

	void ProcessSimpleInParameter(CrossCompiler::AST::FParameterDeclarator* ParameterDeclarator, FInputsBodyContext& BodyContext)
	{
		using namespace CrossCompiler::AST;

		FExpression* Initializer = nullptr;

		bool bRequiresToBeOnInputStruct = IsSemanticUsed(ParameterDeclarator->Semantic);
		if (bRequiresToBeOnInputStruct)
		{
			// Add the member to the input struct
			auto* MemberDeclaratorList = new(Allocator) FDeclaratorList(Allocator, SourceInfo);
			MemberDeclaratorList->Type = CloneType(ParameterDeclarator->Type);
			auto* MemberDeclaration = new(Allocator) FDeclaration(Allocator, SourceInfo);
			MemberDeclaration->Identifier = ParameterDeclarator->Identifier;
			MemberDeclaration->Semantic = new(Allocator) FSemanticSpecifier(Allocator, ParameterDeclarator->Semantic->Semantic, SourceInfo);
			MemberDeclaratorList->Declarations.Add(MemberDeclaration);

			// Add it to the input struct type
			check(BodyContext.NewInputStruct);
			BodyContext.NewInputStruct->Members.Add(MemberDeclaratorList);

			// Make this parameter the initializer of the new local variable
			FString ParameterName = BodyContext.InputVariableName;
			ParameterName += TEXT(".");
			ParameterName += ParameterDeclarator->Identifier->ToStringView();
			Initializer = MakeIdentifierExpression(Allocator, *ParameterName, SourceInfo);
		}
		else
		{
			// Make a local to generate the in parameter: Type Local = (Type)0;
			auto* Zero = new(Allocator) FExpression(Allocator, EOperators::Literal, SourceInfo);
			Zero->LiteralType = CrossCompiler::ELiteralType::Integer;
			Zero->Identifier = FIdentifier::New(Allocator, TEXTVIEW("0"));
			Initializer = new(Allocator) FUnaryExpression(Allocator, EOperators::TypeCast, Zero, SourceInfo);
			Initializer->TypeSpecifier = ParameterDeclarator->Type->Specifier;

			RemovedSemantics.Add(ParameterDeclarator->Semantic->Semantic);
		}

		auto* LocalVar = CreateLocalVariable(ParameterDeclarator->Type->Specifier->TypeName, ParameterDeclarator->Identifier, Initializer);
		BodyContext.PreInstructions.Add(LocalVar);
	}

	void WriteGeneratedInCode(CrossCompiler::AST::FFunctionDefinition* NewEntryFunction, TArray<CrossCompiler::AST::FStructSpecifier*>& NewStructs, FString& OutGeneratedCode)
	{
		CrossCompiler::AST::FASTWriter Writer(OutGeneratedCode);
		GeneratedCode = TEXT("#line 1 \"RemoveUnusedInputs.usf\"\n// Generated Entry Point: ");
		GeneratedCode += NewEntryFunction->Prototype->Identifier->ToStringView();
		GeneratedCode += TEXT("\n");
		if (UsedInputs.Num() > 0)
		{
			GeneratedCode += TEXT("// Requested UsedInputs:");
			for (int32 Index = 0; Index < UsedInputs.Num(); ++Index)
			{
				GeneratedCode += (Index == 0) ? TEXT(" ") : TEXT(", ");
				GeneratedCode += UsedInputs[Index];
			}
			GeneratedCode += TEXT("\n");
		}
		if (RemovedSemantics.Num() > 0)
		{
			GeneratedCode += TEXT("// Removed Inputs:");
			for (int32 Index = 0; Index < RemovedSemantics.Num(); ++Index)
			{
				GeneratedCode += (Index == 0) ? TEXT(" ") : TEXT(", ");
				GeneratedCode += RemovedSemantics[Index];
			}
			GeneratedCode += TEXT("\n");
		}
		for (auto* Struct : NewStructs)
		{
			auto* Declarator = new(Allocator)CrossCompiler::AST::FDeclaratorList(Allocator, SourceInfo);
			Declarator->Declarations.Add(Struct);
			Declarator->Write(Writer);
		}
		NewEntryFunction->Write(Writer);
		//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("*********************************\n%s\n"), *GeneratedCode);
	}

	bool AddUsedInputMembers(CrossCompiler::AST::FStructSpecifier* DestStruct, const TCHAR* DestPrefix, CrossCompiler::AST::FStructSpecifier* SourceStruct, const TCHAR* SourcePrefix, TArray<CrossCompiler::AST::FStructSpecifier*>& MiniSymbolTable, FBodyContext& BodyContext)
	{
		using namespace CrossCompiler::AST;

		for (auto* Member : SourceStruct->Members)
		{
			FDeclaratorList* MemberDeclarator = Member->AsDeclaratorList();
			if (MemberDeclarator)
			{
				for (auto* DeclarationNode : MemberDeclarator->Declarations)
				{
					FDeclaration* MemberDeclaration = DeclarationNode->AsDeclaration();
					check(MemberDeclaration);
					if (MemberDeclaration->Semantic)
					{
						if (MemberDeclaration->bIsArray)
						{
							uint32 ArrayLength = 0;
							if (!GetArrayLength(MemberDeclaration, ArrayLength))
							{
								return false;
							}

							uint32 StartIndex = 0;
							TCHAR* ElementSemanticPrefix = GetNonDigitSemanticPrefix(Allocator, MemberDeclaration->Semantic->Semantic, StartIndex);
							if (!ElementSemanticPrefix)
							{
								Errors.Add(FString::Printf(TEXT("RemoveUnusedInputs: Member (%s) %s : %s is expected to have an indexed semantic!"), MemberDeclarator->Type->Specifier->TypeName, MemberDeclaration->Identifier->GetData(), MemberDeclaration->Semantic->Semantic));

								// Fatal: Array of non-indexed semantic (eg float4 Colors[4] : MYSEMANTIC; )
								// Assume semantic is used and just fallback
								auto* NewDeclaratorList = new(Allocator) FDeclaratorList(Allocator, MemberDeclarator->SourceInfo);
								NewDeclaratorList->Type = CloneType(MemberDeclarator->Type);
								NewDeclaratorList->Declarations.Add(MemberDeclaration);
								DestStruct->Members.Add(NewDeclaratorList);

								// Source and Dest are swapped as we are copying from the optimized (dest) structure into the original (source) structure
								CopyMember(MemberDeclaration, SourcePrefix, DestPrefix, BodyContext.PreInstructions);
							}
							else
							{
								for (uint32 Index = 0; Index < ArrayLength; ++Index)
								{
									TCHAR* ElementSemantic = MakeIndexedSemantic(Allocator, ElementSemanticPrefix, StartIndex + Index);
									if (IsSemanticUsed(ElementSemantic))
									{
										auto* NewMemberDeclaration = new(Allocator) FDeclaration(Allocator, MemberDeclaration->SourceInfo);
										NewMemberDeclaration->Semantic = new(Allocator) FSemanticSpecifier(Allocator, ElementSemantic, MemberDeclaration->SourceInfo);
										NewMemberDeclaration->Identifier = FIdentifier::New(Allocator, FString::Printf(TEXT("%s_%d"), MemberDeclaration->Identifier->GetData(), Index));

										// Add member to struct
										auto* NewDeclaratorList = new(Allocator) FDeclaratorList(Allocator, MemberDeclarator->SourceInfo);
										NewDeclaratorList->Type = CloneType(MemberDeclarator->Type);
										NewDeclaratorList->Declarations.Add(NewMemberDeclaration);
										DestStruct->Members.Add(NewDeclaratorList);

										FString LHSElement = FString::Printf(TEXT("%s.%s[%d]"), SourcePrefix, MemberDeclaration->Identifier->GetData(), Index);
										FString RHSElement = FString::Printf(TEXT("%s.%s"), DestPrefix, NewMemberDeclaration->Identifier->GetData());

										auto* LHS = MakeIdentifierExpression(Allocator, *LHSElement, SourceInfo);
										auto* RHS = MakeIdentifierExpression(Allocator, *RHSElement, SourceInfo);
										auto* Assignment = new(Allocator) FBinaryExpression(Allocator, EOperators::Assign, LHS, RHS, SourceInfo);
										BodyContext.PreInstructions.Add(new(Allocator) FExpressionStatement(Allocator, Assignment, SourceInfo));
									}
									else
									{
										RemovedSemantics.Add(ElementSemantic);
									}
								}
							}
						}
						else if (IsSemanticUsed(MemberDeclaration->Semantic))
						{
							// Add member to struct
							auto* NewDeclaratorList = new(Allocator) FDeclaratorList(Allocator, MemberDeclarator->SourceInfo);
							NewDeclaratorList->Type = CloneType(MemberDeclarator->Type);
							NewDeclaratorList->Declarations.Add(MemberDeclaration);
							DestStruct->Members.Add(NewDeclaratorList);

							// Source and Dest are swapped as we are copying from the optimized (dest) structure into the original (source) structure
							CopyMember(MemberDeclaration, SourcePrefix, DestPrefix, BodyContext.PreInstructions);
						}
						else
						{
							// Ignore as the base struct is zero'd out
	/*
							auto* Zero = new(Allocator) FUnaryExpression(Allocator, EOperators::FloatConstant, nullptr, SourceInfo);
							Zero->FloatConstant = 0;
							auto* Cast = new(Allocator) FUnaryExpression(Allocator, EOperators::TypeCast, Zero, SourceInfo);
							Cast->TypeSpecifier = MemberDeclarator->Type->Specifier;
							auto* Assignment = new(Allocator) FBinaryExpression(Allocator, EOperators::Assign, MakeIdentifierExpression(Allocator, SourcePrefix, SourceInfo), Cast, SourceInfo);
							auto* Statement = new(Allocator) FExpressionStatement(Allocator, Assignment, SourceInfo);
							BodyContext.PostInstructions.Add(Statement);
	*/
							RemovedSemantics.Add(MemberDeclaration->Semantic->Semantic);
						}
					}
					else
					{
						if (!MemberDeclarator->Type || !MemberDeclarator->Type->Specifier || !MemberDeclarator->Type->Specifier->TypeName)
						{
							Errors.Add(FString::Printf(TEXT("RemoveUnusedInputs: Internal error tracking down nested type %s"), MemberDeclaration->Identifier->GetData()));
							return false;
						}

						// No semantic, so make sure this is a nested struct, or error that it's missing a semantic
						FStructSpecifier* NestedStructSpecifier = FindStructSpecifier(MiniSymbolTable, MemberDeclarator->Type->Specifier->TypeName);
						if (!NestedStructSpecifier)
						{
							Errors.Add(FString::Printf(TEXT("RemoveUnusedInputs: Member (%s) %s is expected to have a semantic!"), MemberDeclarator->Type->Specifier->TypeName, MemberDeclaration->Identifier->GetData()));
							return false;
						}

						// Add all the elements of this new struct into the return type
						FString NewSourcePrefix = SourcePrefix;
						NewSourcePrefix += TEXT(".");
						NewSourcePrefix += MemberDeclaration->Identifier->ToStringView();
						AddUsedInputMembers(DestStruct, DestPrefix, NestedStructSpecifier, *NewSourcePrefix, MiniSymbolTable, BodyContext);
					}
				}
			}
			else
			{
				check(0);
			}
		}
		return true;
	}
};

namespace UE::HlslParser
{

bool RemoveUnusedInputs(FString& InOutSourceCode,
	TConstArrayView<FStringView> InUsedInputs,
	TConstArrayView<FScopedDeclarations> InScopedDeclarations,
	FString& InOutEntryPoint,
	TArray<FString>& OutErrors)
{
	FString DummyFilename(TEXT("/Engine/Private/RemoveUnusedInputs.usf"));
	FRemoveUnusedInputs Data(InUsedInputs);
	Data.EntryPoint = InOutEntryPoint;
	CrossCompiler::FCompilerMessages Messages;
	auto Lambda = [&Data](CrossCompiler::FLinearAllocator* Allocator, CrossCompiler::TLinearArray<CrossCompiler::AST::FNode*>& ASTNodes)
	{
		Data.Allocator = Allocator;
		Data.RemoveUnusedInputs(ASTNodes);
		if (!Data.bSuccess)
		{
			int i = 0;
			++i;
		}
	};
	if (!CrossCompiler::Parser::Parse(InOutSourceCode, DummyFilename, Messages, InScopedDeclarations, Lambda))
	{
		Data.Errors.Add(FString(TEXT("RemoveUnusedInputs: Failed to compile!")));
		OutErrors = Data.Errors;
		for (auto& Message : Messages.MessageList)
		{
			OutErrors.Add(Message.Message);
		}
		return false;
	}

	for (auto& Message : Messages.MessageList)
	{
		OutErrors.Add(Message.Message);
	}

	if (Data.bSuccess)
	{
		InOutSourceCode += (TCHAR)'\n';
		InOutSourceCode += Data.GeneratedCode;
		InOutEntryPoint = Data.EntryPoint;

		return true;
	}

	OutErrors = Data.Errors;
	return false;
}

bool RemoveUnusedInputs(FString& InOutSourceCode, const TArray<FString>& InInputs, FString& EntryPoint, TArray<FString>& OutErrors)
{
	const TArray<FStringView> Inputs(MakeArrayView(InInputs));
	return RemoveUnusedInputs(InOutSourceCode, Inputs, {}, EntryPoint, OutErrors);
}

} // namespace UE::HlslParser

struct FFindEntryPointParameters
{
	FString EntryPoint;
	const bool bFindOutputSemantics;

	bool bSuccess;
	TArray<FString> Errors;
	CrossCompiler::FLinearAllocator* Allocator;

	TArray<FString> FoundSemantics;

	FFindEntryPointParameters(const FStringView& EntryPoint, UE::HlslParser::EShaderParameterStorageClass SemanticsStorageClass) :
		EntryPoint(EntryPoint),
		bFindOutputSemantics(SemanticsStorageClass == UE::HlslParser::EShaderParameterStorageClass::Output),
		bSuccess(false),
		Allocator(nullptr)
	{
	}

	CrossCompiler::AST::FFunctionDefinition* FindEntryPointAndPopulateSymbolTable(CrossCompiler::TLinearArray<CrossCompiler::AST::FNode*>& ASTNodes, TArray<CrossCompiler::AST::FStructSpecifier*>& OutMiniSymbolTable)
	{
		using namespace CrossCompiler::AST;
		FFunctionDefinition* EntryFunction = nullptr;
		for (int32 Index = 0; Index < ASTNodes.Num(); ++Index)
		{
			FNode* Node = ASTNodes[Index];
			if (FDeclaratorList* DeclaratorList = Node->AsDeclaratorList())
			{
				if (FStructSpecifier* StructSpecifier = DeclaratorList->Type->Specifier->Structure)
				{
					// Skip unnamed structures
					if (StructSpecifier->Name)
					{
						OutMiniSymbolTable.Add(StructSpecifier);
					}
				}
			}
			else if (FFunctionDefinition* FunctionDefinition = Node->AsFunctionDefinition())
			{
				if (FunctionDefinition->Prototype->Identifier->Equals(EntryPoint))
				{
					EntryFunction = FunctionDefinition;
				}
			}
		}

		return EntryFunction;
	}

	CrossCompiler::AST::FStructSpecifier* FindStructSpecifier(TArray<CrossCompiler::AST::FStructSpecifier*>& MiniSymbolTable, const TCHAR* StructName)
	{
		for (auto* StructSpecifier : MiniSymbolTable)
		{
			if (!FCString::Strcmp(StructSpecifier->Name, StructName))
			{
				return StructSpecifier;
			}
		}
		return nullptr;
	}

	bool ProcessParameterSemantic(CrossCompiler::AST::FSemanticSpecifier& SemanticSpecifier, const TCHAR* ParameterIdentifier, bool bIsOutput)
	{
		if (const TCHAR* SemanticName = SemanticSpecifier.Semantic)
		{
			if (bFindOutputSemantics == bIsOutput)
			{
				FoundSemantics.Add(SemanticName);
			}
			return true;
		}
		else
		{
			check(ParameterIdentifier);
			Errors.Add(FString::Printf(TEXT("FindEntryPointParameters: Function %s with parameter %s doesn't have a valid semantic name"), *EntryPoint, ParameterIdentifier));
			return false;
		}
	}

	bool ProcessStructParameterSemantics(CrossCompiler::AST::FStructSpecifier& StructSpecifier, TArray<CrossCompiler::AST::FStructSpecifier*>& SymbolTable, bool bIsOutput)
	{
		using namespace CrossCompiler::AST;
		for (FNode* MemberNode : StructSpecifier.Members)
		{
			FDeclaratorList* DeclList = MemberNode->AsDeclaratorList();
			if (!DeclList)
			{
				continue;
			}
			for (FNode* MemberNodeDecl : DeclList->Declarations)
			{
				if (FDeclaration* MemberDecl = MemberNodeDecl->AsDeclaration())
				{
					if (FSemanticSpecifier* Semantic = MemberDecl->Semantic)
					{
						if (!ProcessParameterSemantic(*Semantic, MemberDecl->Identifier->GetData(), bIsOutput))
						{
							return false;
						}
					}
					else if (FStructSpecifier* SubStructSpecifier = FindStructSpecifier(SymbolTable, DeclList->Type->Specifier->TypeName))
					{
						if (!ProcessStructParameterSemantics(*SubStructSpecifier, SymbolTable, bIsOutput))
						{
							return false;
						}
					}
					else if (CheckSimpleVectorType(DeclList->Type->Specifier->TypeName))
					{
						Errors.Add(FString::Printf(TEXT("FindEntryPointParameters: Function %s with parameter %s doesn't have a return semantic"), *EntryPoint, MemberDecl->Identifier->GetData()));
						return false;
					}
					else
					{
						Errors.Add(FString::Printf(TEXT("FindEntryPointParameters: Invalid return type %s for parameter %s in function %s"), DeclList->Type->Specifier->TypeName, MemberDecl->Identifier->GetData(), *EntryPoint));
						return false;
					}
				}
			}
		}
		return true;
	}

	bool ProcessFunctionParameters(CrossCompiler::AST::FFunctionDefinition* EntryFunction, TArray<CrossCompiler::AST::FStructSpecifier*>& SymbolTable)
	{
		using namespace CrossCompiler::AST;
		for (FNode* ParamNode : EntryFunction->Prototype->Parameters)
		{
			FParameterDeclarator* ParameterDeclarator = ParamNode->AsParameterDeclarator();
			check(ParameterDeclarator);
			const bool bIsOutput = ParameterDeclarator->Type->Qualifier.bOut;
			if (FSemanticSpecifier* Semantic = ParameterDeclarator->Semantic)
			{
				if (!ProcessParameterSemantic(*Semantic, ParameterDeclarator->Identifier->GetData(), bIsOutput))
				{
					return false;
				}
			}
			else if (FStructSpecifier* StructSpecifier = FindStructSpecifier(SymbolTable, ParameterDeclarator->Type->Specifier->TypeName))
			{
				if (!ProcessStructParameterSemantics(*StructSpecifier, SymbolTable, bIsOutput))
				{
					return false;
				}
			}
			else if (CheckSimpleVectorType(ParameterDeclarator->Type->Specifier->TypeName))
			{
				Errors.Add(FString::Printf(TEXT("FindEntryPointParameters: Function %s with parameter %s doesn't have a return semantic"), *EntryPoint, ParameterDeclarator->Identifier->GetData()));
				return false;
			}
			else
			{
				Errors.Add(FString::Printf(TEXT("FindEntryPointParameters: Invalid return type %s for parameter %s in function %s"), ParameterDeclarator->Type->Specifier->TypeName, ParameterDeclarator->Identifier->GetData(), *EntryPoint));
				return false;
			}
		}
		return true;
	}

	void FindEntryPointParameters(CrossCompiler::TLinearArray<CrossCompiler::AST::FNode*>& ASTNodes)
	{
		using namespace CrossCompiler::AST;

		// Find Entry point from original AST nodes
		TArray<FStructSpecifier*> SymbolTable;
		FFunctionDefinition* EntryFunction = FindEntryPointAndPopulateSymbolTable(ASTNodes, SymbolTable);
		if (!EntryFunction)
		{
			Errors.Add(FString::Printf(TEXT("FindEntryPointParameters: Unable to find entry point %s"), *EntryPoint));
			bSuccess = false;
			return;
		}

		if (!ProcessFunctionParameters(EntryFunction, SymbolTable))
		{
			bSuccess = false;
			return;
		}

		bSuccess = true;
	}
};

namespace UE::HlslParser
{

bool FindEntryPointParameters(
	const FString& InSourceCode,
	const FString& InEntryPoint,
	EShaderParameterStorageClass ParameterStorageClass,
	TConstArrayView<FScopedDeclarations> InScopedDeclarations,
	TArray<FString>& OutParameterSemantics,
	TArray<FString>& OutErrors)
{
	check(ParameterStorageClass == EShaderParameterStorageClass::Input || ParameterStorageClass == EShaderParameterStorageClass::Output);

	FString DummyFilename(TEXT("/Engine/Private/FindEntryPointParameters.usf"));
	FFindEntryPointParameters Data(InEntryPoint, ParameterStorageClass);
	auto ResultCallbackFunction = [&Data](CrossCompiler::FLinearAllocator* Allocator, CrossCompiler::TLinearArray<CrossCompiler::AST::FNode*>& ASTNodes) -> void
		{
			Data.Allocator = Allocator;
			Data.FindEntryPointParameters(ASTNodes);
		};
	CrossCompiler::FCompilerMessages Messages;
	if (!CrossCompiler::Parser::Parse(InSourceCode, DummyFilename, Messages, InScopedDeclarations, ResultCallbackFunction))
	{
		Data.Errors.Add(FString(TEXT("FindEntryPointParameters: Failed to parse HLSL source!")));
		OutErrors = MoveTemp(Data.Errors);
		for (auto& Message : Messages.MessageList)
		{
			OutErrors.Add(Message.Message);
		}
		return false;
	}

	for (auto& Message : Messages.MessageList)
	{
		OutErrors.Add(Message.Message);
	}

	if (Data.bSuccess)
	{
		OutParameterSemantics = MoveTemp(Data.FoundSemantics);
		return true;
	}

	OutErrors = MoveTemp(Data.Errors);
	return false;
}

bool ForEachHlslIdentifier(const FString& InSourceCode, const FString& InFilename, const TFunction<bool(FStringView)>& InIdentifierCallback)
{
	CrossCompiler::FCompilerMessages Messages;
	CrossCompiler::FHlslScanner Scanner(Messages);
	if (!Scanner.Lex(InSourceCode, InFilename))
	{
		return false;
	}

	while (Scanner.HasMoreTokens())
	{
		if (const CrossCompiler::FHlslToken* HlslToken = Scanner.GetCurrentToken())
		{
			if (HlslToken->Token == CrossCompiler::EHlslToken::Identifier)
			{
				// Report identifier to callback and stop iterating once the callback returns false
				if (!InIdentifierCallback(HlslToken->String))
				{
					break;
				}
			}
		}
		Scanner.Advance();
	}

	return true;
}

} // namespace CrossCompiler

struct FConvertFP32ToFP16 {
	FString Filename;
	FString GeneratedCode;
	bool bSuccess;
};

static FStringView FindFP16Type(const FStringView TypeName)
{
	static const FStringView FloatTypes[9] = { TEXTVIEW("float"), TEXTVIEW("float2"), TEXTVIEW("float3"), TEXTVIEW("float4"), TEXTVIEW("float2x2"), TEXTVIEW("float3x3"), TEXTVIEW("float4x4"), TEXTVIEW("float3x4"), TEXTVIEW("float4x3") };
	static const FStringView HalfTypes[9] = { TEXTVIEW("half"), TEXTVIEW("half2"), TEXTVIEW("half3"), TEXTVIEW("half4"), TEXTVIEW("half2x2"), TEXTVIEW("half3x3"), TEXTVIEW("half4x4"), TEXTVIEW("half3x4"), TEXTVIEW("half4x3") };
	//static const FStringView HalfTypes[9] = { TEXTVIEW("min16float"), TEXTVIEW("min16float2"), TEXTVIEW("min16float3"), TEXTVIEW("min16float4"), TEXTVIEW("min16float2x2"), TEXTVIEW("min16float3x3"), TEXTVIEW("min16float4x4"), TEXTVIEW("min16float3x4"), TEXTVIEW("min16float4x3") };
	for (int32 i = 0; i < 9; ++i)
	{
		if (TypeName.Equals(FloatTypes[i]))
		{
			return HalfTypes[i];
		}
	}

	return {};
}

static void ConvertFromFP32ToFP16(const CrossCompiler::AST::FIdentifier*& IdentifierName, CrossCompiler::FLinearAllocator* Allocator)
{
	FStringView NewType = FindFP16Type(IdentifierName->ToStringView());
	if (NewType.IsEmpty()) return;
	check(Allocator);
	IdentifierName = CrossCompiler::AST::FIdentifier::New(Allocator, NewType);
}

static void ConvertFromFP32ToFP16(CrossCompiler::AST::FTypeSpecifier* Type, CrossCompiler::FLinearAllocator* Allocator)
{
	FStringView NewType = FindFP16Type(FStringView(Type->TypeName));
	if (NewType.IsEmpty()) return;
	check(Allocator);
	Type->TypeName = Allocator->Strdup(NewType.GetData(), NewType.Len());
}

static void ConvertFromFP32ToFP16Base(CrossCompiler::AST::FNode* Node, CrossCompiler::FLinearAllocator* Allocator);
static void ConvertFromFP32ToFP16(CrossCompiler::AST::FFunctionDefinition* Node, CrossCompiler::FLinearAllocator* Allocator)
{
	if (Node->Prototype->Identifier->Equals(TEXTVIEW("CalcSceneDepth")))
	{
		return;
	}
	ConvertFromFP32ToFP16(Node->Prototype->ReturnType->Specifier, Allocator);
	for (auto Elem : Node->Prototype->Parameters)
	{
		ConvertFromFP32ToFP16Base(Elem, Allocator);
	}
	for (auto Elem : Node->Body->Statements)
	{
		ConvertFromFP32ToFP16Base(Elem, Allocator);
	}
}

//For these functions we do not convert arrays as no implicit conversion is allowed between half and float for arrays
static void ConvertFromFP32ToFP16(CrossCompiler::AST::FParameterDeclarator* Node, CrossCompiler::FLinearAllocator* Allocator)
{
	if (Node->bIsArray)
	{
		if (!Node->Identifier->Equals(TEXTVIEW("MRT")))
		{
			return;
		}
	}
	ConvertFromFP32ToFP16(Node->Type->Specifier, Allocator);
}

static void ConvertFromFP32ToFP16(CrossCompiler::AST::FDeclaratorList* Node, CrossCompiler::FLinearAllocator* Allocator)
{
	for (auto Elem : Node->Declarations)
	{
		if (Elem->AsDeclaration() && Elem->AsDeclaration()->bIsArray)
		{
			if (!Elem->AsDeclaration()->Identifier->Equals(TEXTVIEW("MRT")))
			{
				return;
			}
		}
	}
	ConvertFromFP32ToFP16(Node->Type->Specifier, Allocator);
}

static void ConvertFromFP32ToFP16(CrossCompiler::AST::FSelectionStatement* Node, CrossCompiler::FLinearAllocator* Allocator)
{
	if (Node->ThenStatement)
	{
		ConvertFromFP32ToFP16Base(Node->ThenStatement, Allocator);
	}
	if (Node->ElseStatement)
	{
		ConvertFromFP32ToFP16Base(Node->ElseStatement, Allocator);
	}
}

static void ConvertFromFP32ToFP16(CrossCompiler::AST::FIterationStatement* Node, CrossCompiler::FLinearAllocator* Allocator)
{
	if (Node->InitStatement)
	{
		ConvertFromFP32ToFP16Base(Node->InitStatement, Allocator);
	}
	if (Node->Condition)
	{
		ConvertFromFP32ToFP16Base(Node->Condition, Allocator);
	}
	if (Node->Body)
	{
		ConvertFromFP32ToFP16Base(Node->Body, Allocator);
	}
}

static void ConvertFromFP32ToFP16(CrossCompiler::AST::FCompoundStatement* Node, CrossCompiler::FLinearAllocator* Allocator)
{
	for (auto Statement : Node->Statements)
	{
		ConvertFromFP32ToFP16Base(Statement, Allocator);
	}
}

static void ConvertFromFP32ToFP16(CrossCompiler::AST::FSwitchStatement* Node, CrossCompiler::FLinearAllocator* Allocator)
{
	if (Node->Body == nullptr || Node->Body->CaseList == nullptr)
	{
		return;
	}
	for (auto Elem : Node->Body->CaseList->Cases)
	{
		if (Elem == nullptr)
		{
			continue;
		}
		for (auto Statement : Elem->Statements)
		{
			if (Statement)
			{
				ConvertFromFP32ToFP16Base(Statement, Allocator);
			}
		}
	}
}

static void ConvertFromFP32ToFP16(CrossCompiler::AST::FExpression* Expression, CrossCompiler::FLinearAllocator* Allocator)
{
    if (Expression->Operator == CrossCompiler::AST::EOperators::Identifier)
    {
        ConvertFromFP32ToFP16(Expression->Identifier, Allocator);
    }
    if (Expression->Operator == CrossCompiler::AST::EOperators::TypeCast)
    {
        ConvertFromFP32ToFP16(Expression->TypeSpecifier, Allocator);
    }
    if (Expression->Operator == CrossCompiler::AST::EOperators::FieldSelection)
    {
        ConvertFromFP32ToFP16(Expression->Expressions[0], Allocator);
    }
    if (Expression->Operator == CrossCompiler::AST::EOperators::Assign)
    {
		ConvertFromFP32ToFP16(Expression->Expressions[0], Allocator);
		ConvertFromFP32ToFP16(Expression->Expressions[1], Allocator);
    }
    if (Expression->Operator == CrossCompiler::AST::EOperators::FunctionCall)
    {
        if (Expression->Expressions[0])
        {
            ConvertFromFP32ToFP16(Expression->Expressions[0], Allocator);
        }
        for (auto SubExpression : Expression->Expressions)
        {
            ConvertFromFP32ToFP16(SubExpression, Allocator);
        }
    }
}

static void ConvertFromFP32ToFP16(CrossCompiler::AST::FExpressionStatement* Node, CrossCompiler::FLinearAllocator* Allocator)
{
    if (Node->Expression == nullptr)
    {
        return;
    }
    ConvertFromFP32ToFP16(Node->Expression, Allocator);
}

static void ConvertFromFP32ToFP16(CrossCompiler::AST::FJumpStatement* Node, CrossCompiler::FLinearAllocator* Allocator)
{
    if (Node->OptionalExpression == nullptr)
    {
        return;
    }
    ConvertFromFP32ToFP16(Node->OptionalExpression, Allocator);
}

static void ConvertFromFP32ToFP16Base(CrossCompiler::AST::FNode* Node, CrossCompiler::FLinearAllocator* Allocator)
{
	if (Node->AsFunctionDefinition())
	{
		ConvertFromFP32ToFP16(Node->AsFunctionDefinition(), Allocator);
	}
	if (Node->AsParameterDeclarator())
	{
		ConvertFromFP32ToFP16(Node->AsParameterDeclarator(), Allocator);
	}
	if (Node->AsDeclaratorList())
	{
		ConvertFromFP32ToFP16(Node->AsDeclaratorList(), Allocator);
	}
	if (Node->AsSelectionStatement())
	{
		ConvertFromFP32ToFP16(Node->AsSelectionStatement(), Allocator);
	}
	if (Node->AsSwitchStatement())
	{
		ConvertFromFP32ToFP16(Node->AsSwitchStatement(), Allocator);
	}
	if (Node->AsIterationStatement())
	{
		ConvertFromFP32ToFP16(Node->AsIterationStatement(), Allocator);
	}
	if (Node->AsCompoundStatement())
	{
		ConvertFromFP32ToFP16(Node->AsCompoundStatement(), Allocator);
	}
    if (Node->AsExpressionStatement())
    {
		ConvertFromFP32ToFP16(Node->AsExpressionStatement(), Allocator);
    }
    if (Node->AsJumpStatement())
    {
		ConvertFromFP32ToFP16(Node->AsJumpStatement(), Allocator);
    }
}

static void ConvertFromFP32ToFP16(CrossCompiler::AST::FStructSpecifier* Node, CrossCompiler::FLinearAllocator* Allocator)
{
    for (auto Member : Node->Members)
    {
        ConvertFromFP32ToFP16Base(Member, Allocator);
    }
}

static void HlslParserCallbackWrapperFP32ToFP16(void* CallbackData, CrossCompiler::FLinearAllocator* Allocator, CrossCompiler::TLinearArray<CrossCompiler::AST::FNode*>& ASTNodes)
{
	auto* ConvertData = (FConvertFP32ToFP16*)CallbackData;
	CrossCompiler::AST::FASTWriter writer(ConvertData->GeneratedCode);
	TMap<FString, bool> GlobalStructures;
	for (auto Elem : ASTNodes)
	{
		// We find all structures that are used for global vars and add them to the list of ones we cannot change
		if (Elem->AsParameterDeclarator())
		{
			GlobalStructures.Add(Elem->AsParameterDeclarator()->Type->Specifier->TypeName);
		}
		if (Elem->AsDeclaratorList())
		{
			GlobalStructures.Add(Elem->AsDeclaratorList()->Type->Specifier->TypeName);
		}
	}
	for (auto Elem : ASTNodes)
	{
		if (Elem->AsFunctionDefinition())
		{
			ConvertFromFP32ToFP16(Elem->AsFunctionDefinition(), Allocator);
		}
		if (Elem->AsDeclaratorList() && Elem->AsDeclaratorList()->Type && Elem->AsDeclaratorList()->Type->Specifier->Structure)
		{
			if (!GlobalStructures.Contains(Elem->AsDeclaratorList()->Type->Specifier->Structure->Name))
			{
				ConvertFromFP32ToFP16(Elem->AsDeclaratorList()->Type->Specifier->Structure, Allocator);
			}
		}
		Elem->Write(writer);
	}
	ConvertData->bSuccess = true;
}

bool ConvertFromFP32ToFP16(FString& InOutSourceCode, TArray<FString>& OutErrors)
{
	FString DummyFilename(TEXT("/Engine/Private/ConvertFP32ToFP16.usf"));
	CrossCompiler::FCompilerMessages Messages;
	FConvertFP32ToFP16 Data;
	Data.Filename = DummyFilename;
	Data.GeneratedCode = "";
	if (!CrossCompiler::Parser::Parse(InOutSourceCode, DummyFilename, Messages, {}, HlslParserCallbackWrapperFP32ToFP16, &Data))
	{
		OutErrors.Add(FString(TEXT("ConvertFP32ToFP16: Failed to compile!")));
		for (auto& Message : Messages.MessageList)
		{
			OutErrors.Add(Message.Message);
		}
		return false;
	}

	for (auto& Message : Messages.MessageList)
	{
		OutErrors.Add(Message.Message);
	}

	if (Data.bSuccess)
	{
		InOutSourceCode = Data.GeneratedCode;
		return true;
	}

	return false;
}

namespace UE::ShaderCompilerCommon
{
	static void BuildDiagnosticsSourceInfoMsvc(TStringBuilder<1024>& DiagnosticsStringBuilder, const CrossCompiler::FSourceInfo& SourceInfo)
	{
		DiagnosticsStringBuilder.Appendf(TEXT("%s(%d,%d): "), SourceInfo.Filename != nullptr ? **SourceInfo.Filename : TEXT("<unknown>"), SourceInfo.Line, SourceInfo.Column);
	}

	bool ValidateShaderAgainstKnownIssues(const FString& InSourceCode, TArray<FString>& OutErrors, const TCHAR* InSourceCodeFilename)
	{
		struct FShaderValidationContext
		{
			TArray<FString> Errors;

			void Error(const FString& InMessage, const CrossCompiler::FSourceInfo* SourceInfo = nullptr)
			{
				if (SourceInfo)
				{
					TStringBuilder<1024> DiagnosticsStringBuilder;
					BuildDiagnosticsSourceInfoMsvc(DiagnosticsStringBuilder, *SourceInfo);
					Errors.Add(FString::Printf(TEXT("%s%s"), DiagnosticsStringBuilder.ToString(), *InMessage));
				}
				else
				{
					Errors.Add(InMessage);
				}
			}

			void ValidateDeclarationIdentifier(const CrossCompiler::AST::FIdentifier* InIdentifier, const CrossCompiler::FSourceInfo* SourceInfo = nullptr)
			{
				// Validate identifier against fixed set of names that are known to cause problems with FXC
				if (InIdentifier->Equals(TEXTVIEW("sample")))
				{
					Error(FString::Printf(TEXT("Identifier \"%s\" must not be used as parameter or declaration identifier. FXC (D3D11) will misinterpret it as interpolation qualifier type. Consider renaming it to \"Sample\"."), InIdentifier->GetData()), SourceInfo);
				}
				if (InIdentifier->Equals(TEXTVIEW("Buffer")))
				{
					Error(FString::Printf(TEXT("Identifier \"%s\" must not be used as parameter or declaration identifier. FXC (D3D11) will misinterpret it as resource type. Consider adding the \"In\" or \"Out\" prefix."), InIdentifier->GetData()), SourceInfo);
				}
			}

			void ValidateFunctionDefinition(CrossCompiler::AST::FFunctionDefinition* FunctionDef)
			{
				check(FunctionDef->Prototype != nullptr);
				for (CrossCompiler::AST::FNode* Parameter : FunctionDef->Prototype->Parameters)
				{
					Validate(Parameter);
				}
			}

			void Validate(CrossCompiler::AST::FNode* Node)
			{
				check(Node);
				// Validate parameter and variable identifiers
				if (CrossCompiler::AST::FParameterDeclarator* ParameterDecl = Node->AsParameterDeclarator())
				{
					ValidateDeclarationIdentifier(ParameterDecl->Identifier, &ParameterDecl->SourceInfo);
				}
				else if (CrossCompiler::AST::FDeclaration* Decl = Node->AsDeclaration())
				{
					ValidateDeclarationIdentifier(Decl->Identifier, &Decl->SourceInfo);
				}
				else if (CrossCompiler::AST::FFunctionDefinition* FunctionDef = Node->AsFunctionDefinition())
				{
					ValidateFunctionDefinition(FunctionDef);
				}
			}
		};

		FShaderValidationContext ValidationContext;

		auto ValidateASTLambda = [&ValidationContext](CrossCompiler::FLinearAllocator* Allocator, CrossCompiler::TLinearArray<CrossCompiler::AST::FNode*>& ASTNodes)
			{
				for (CrossCompiler::AST::FNode* Node : ASTNodes)
				{
					ValidationContext.Validate(Node);
				}
			};

		CrossCompiler::FCompilerMessages Messages;
		const TCHAR* Filename = InSourceCodeFilename != nullptr ? InSourceCodeFilename : TEXT("ShaderValidation.Transient.usf");
		if (!CrossCompiler::Parser::Parse(InSourceCode, Filename, Messages, {}, ValidateASTLambda))
		{
			OutErrors.Add(FString::Printf(TEXT("ValidateShaderAgainstKnownIssues: Failed to parse input file: %s"), Filename));
			for (auto& Message : Messages.MessageList)
			{
				OutErrors.Add(Message.Message);
			}
			return false;
		}

		// Return validation results
		const bool bSuccess = ValidationContext.Errors.IsEmpty();
		OutErrors = MoveTemp(ValidationContext.Errors);
		return bSuccess;
	}
}

static void VisitCompoundStatementChildren(
	CrossCompiler::AST::FCompoundStatement* InRoot,
	TFunction<void(CrossCompiler::AST::FNode*& Node)> Visitor)
{
	using namespace CrossCompiler::AST;

	TArray<FCompoundStatement*, TInlineAllocator<128>> Stack;
	Stack.Push(InRoot);

	auto VisitAndConditionallyPushNode = [&Stack, &Visitor](FNode*& Node)
	{
		if (!Node)
		{
			return false;
		}

		bool bResult = false;
		if (FCompoundStatement* CompoundStatement = Node->AsCompoundStatement())
		{
			Stack.Push(CompoundStatement);
			bResult = true;
		}

		Visitor(Node);

		return bResult;
	};

	while (!Stack.IsEmpty())
	{
		FCompoundStatement* Node = Stack.Last();
		Stack.Pop();

		for (FNode*& ChildStatement : Node->Statements)
		{
			if (VisitAndConditionallyPushNode(ChildStatement))
			{
				continue;
			}

			if (FSelectionStatement* SelectionStatement = ChildStatement->AsSelectionStatement())
			{
				while (SelectionStatement)
				{
					VisitAndConditionallyPushNode(SelectionStatement->ThenStatement);
					VisitAndConditionallyPushNode(SelectionStatement->ElseStatement);

					// Handle chain of `else if` statements
					SelectionStatement = SelectionStatement->ElseStatement ? SelectionStatement->ElseStatement->AsSelectionStatement() : nullptr;
				}
			}
			else if (FSwitchStatement* SwitchStatement = ChildStatement->AsSwitchStatement())
			{
				if (SwitchStatement->Body && SwitchStatement->Body->CaseList)
				{
					for (FCaseStatement* CaseStatement : SwitchStatement->Body->CaseList->Cases)
					{
						for (FNode*& CaseStatementChildNode : CaseStatement->Statements)
						{
							VisitAndConditionallyPushNode(CaseStatementChildNode);
						}
					}
				}
			}
			else if (FIterationStatement* IterationStatement = ChildStatement->AsIterationStatement())
			{
				VisitAndConditionallyPushNode(IterationStatement->Body);
			}
		}
	}
}

static void VisitFunctionIdentifiers(
	CrossCompiler::AST::FFunctionDefinition* FunctionDefinition,
	TFunction<void(CrossCompiler::AST::FIdentifier* Identifier)> ProcessIdentifier)
{
	using namespace CrossCompiler::AST;

	auto ProcessDeclaration = [&ProcessIdentifier](FDeclaration* DeclarationNode)
	{
		if (!DeclarationNode || !DeclarationNode->Identifier)
		{
			return;
		}

		ProcessIdentifier(DeclarationNode->Identifier);
	};

	auto ProcessDeclaratorList = [&ProcessDeclaration](FDeclaratorList* DeclaratorListNode)
	{
		for (FNode* InnerDeclarationNode : DeclaratorListNode->Declarations)
		{
			if (FDeclaration* InnerDeclaration = InnerDeclarationNode->AsDeclaration())
			{
				ProcessDeclaration(InnerDeclaration);
			}
		}
	};

	for (FNode* ParameterNode : FunctionDefinition->Prototype->Parameters)
	{
		FParameterDeclarator* ParameterDeclarator = ParameterNode->AsParameterDeclarator();
		if (!ParameterDeclarator)
		{
			continue;
		}

		if (ParameterDeclarator->Identifier)
		{
			ProcessIdentifier(ParameterDeclarator->Identifier);
		}
	}

	VisitCompoundStatementChildren(FunctionDefinition->Body, [&ProcessDeclaratorList, &ProcessDeclaration](FNode*& StatementNode)
	{
		if (FIterationStatement* IterationStatement = StatementNode->AsIterationStatement())
		{
			FNode* InitStatementNode = IterationStatement->InitStatement;
			if (FDeclaratorList* DeclaratorListNode = InitStatementNode->AsDeclaratorList())
			{
				ProcessDeclaratorList(DeclaratorListNode);
			}
		}
		else if (FDeclaration* Declaration = StatementNode->AsDeclaration())
		{
			ProcessDeclaration(Declaration);
		}
		else if (FDeclaratorList* DeclaratorListNode = StatementNode->AsDeclaratorList())
		{
			ProcessDeclaratorList(DeclaratorListNode);
		}
	});
}

static FString FormatSourceMessage(const CrossCompiler::FSourceInfo& SourceInfo, const TCHAR* Message)
{
	const TCHAR* FileName = (SourceInfo.Filename && !SourceInfo.Filename->IsEmpty()) ? **SourceInfo.Filename : TEXT("<unknown>");
	return FString::Printf(TEXT("%s(%d,%d): %s\n"), FileName, SourceInfo.Line, SourceInfo.Column, Message);
}

bool InlineFunction(FString& InOutSourceCode, FString& InOutEntryPoint, const FStringView FunctionToInline, TArray<FString>& OutErrors)
{
	using namespace CrossCompiler::AST;

	CrossCompiler::FNodeContainer ShaderNodes;

	CrossCompiler::FCompilerMessages CompilerMessages;
	TArray<UE::HlslParser::FScopedDeclarations> ScopedDeclarations;
	if (!CrossCompiler::Parser::Parse(InOutSourceCode, TEXT("inliner.hlsl"), CompilerMessages, ScopedDeclarations, ShaderNodes))
	{
		for (const auto& Message : CompilerMessages.MessageList)
		{
			if (Message.bIsError)
			{
				OutErrors.Add(Message.Message);
			}
		}
		return false;
	}

	auto FindFirstFunctionByName = [&ShaderNodes](FStringView Name) -> FFunctionDefinition*
		{
			for (FNode* Node : ShaderNodes.Nodes)
			{
				FFunctionDefinition* FunctionDefinition = Node->AsFunctionDefinition();
				if (FunctionDefinition && FunctionDefinition->Body && FunctionDefinition->Prototype->Identifier->Equals(Name))
				{
					return FunctionDefinition;
				}
			}

			return nullptr;
		};

	FFunctionDefinition* EntryFunctionDefinition = FindFirstFunctionByName(InOutEntryPoint);
	if (!EntryFunctionDefinition)
	{
		OutErrors.Add(TEXT("Could not find entry point function definition"));
		return false;
	}

	FFunctionDefinition* InlinedFunctionDefinition = FindFirstFunctionByName(FunctionToInline);
	if (!InlinedFunctionDefinition || !InlinedFunctionDefinition->Body)
	{
		OutErrors.Add(TEXT("Could not find inlined function definition"));
		return false;
	}

	// Check that the inlined function meets the current big limitation: no return statements.

	bool bInlinedFunctionCompatible = true;
	VisitCompoundStatementChildren(InlinedFunctionDefinition->Body, [&bInlinedFunctionCompatible, &OutErrors] (FNode*& Node)
	{
		if (FJumpStatement* JumpStatement = Node->AsJumpStatement())
		{
			if (JumpStatement->Type == EJumpType::Return)
			{
				OutErrors.Add(FormatSourceMessage(Node->SourceInfo, TEXT("Inlined functions may not have return statements. Out parameters must be used instead.")));
				bInlinedFunctionCompatible = false;
			}
		}
	});

	if (!bInlinedFunctionCompatible)
	{
		return false;
	}
	
	auto AddPrefixToIdentifier = [&ShaderNodes](FIdentifier* Identifier)
	{
		TStringBuilder<256> NewName;
		NewName << TEXT("INLINE_");
		NewName << Identifier->ToStringView();
		Identifier->Rename(&ShaderNodes.Allocator, NewName);
	};

	// TODO: undo the rename if we need to inline the function multiple times
	// TODO: could only rename colliding names
	VisitFunctionIdentifiers(InlinedFunctionDefinition, AddPrefixToIdentifier);

	CrossCompiler::FNodeContainer OutputNodes;

	auto AsCallToInlinedFunction = [&FunctionToInline](FNode* Statement) -> FExpressionStatement*
		{
			if (FExpressionStatement* ExpressionStatement = Statement->AsExpressionStatement())
			{
				if (ExpressionStatement->Expression->Operator == EOperators::FunctionCall)
				{
					FFunctionExpression* FunctionExpression = static_cast<FFunctionExpression*>(ExpressionStatement->Expression);
					if (FunctionExpression->Callee->Identifier->Equals(FunctionToInline))
					{
						return ExpressionStatement;
					}
				}
			}

			return nullptr;
		};

	const CrossCompiler::FSourceInfo SourceInfo; // todo: more accurate/better one?

	FFunctionDefinition* OutputFunction = OutputNodes.AllocNode<FFunctionDefinition>(SourceInfo);

	TStringBuilder<256> OptimizedEntryPoint;
	OptimizedEntryPoint << EntryFunctionDefinition->Prototype->Identifier->ToStringView();
	OptimizedEntryPoint << TEXTVIEW("__INLINED");

	OutputFunction->Prototype = EntryFunctionDefinition->Prototype;
	OutputFunction->Prototype->Identifier = FIdentifier::New(&OutputNodes.Allocator, OptimizedEntryPoint);
	OutputFunction->Body = EntryFunctionDefinition->Body; // rewrite the original nodes

	// copy over original statements until we hit a call to the function we're inlining

	VisitCompoundStatementChildren(OutputFunction->Body,
		[&bInlinedFunctionCompatible, &AsCallToInlinedFunction, &OutputNodes, &SourceInfo, InlinedFunctionDefinition]
		(FNode*& Node)
	{
		if (FExpressionStatement* CallExpressionStatement = AsCallToInlinedFunction(Node))
		{
			FCompoundStatement* InlinedBody = OutputNodes.AllocNode<FCompoundStatement>(SourceInfo);

			const FExpression* CallExpression = CallExpressionStatement->Expression;

			// add declarations for function paramters
			const int32 NumParemeters = InlinedFunctionDefinition->Prototype->Parameters.Num();
			const int32 NumProvidedParameters = CallExpression ? CallExpression->Expressions.Num() : 0;

			struct FOutParameter
			{
				FParameterDeclarator* Declarator = nullptr;
				FExpression* Expression = nullptr;
			};

			TArray<FOutParameter> OutParameters;

			for (int32 ParamIndex = 0; ParamIndex < NumParemeters; ++ParamIndex)
			{
				FNode* FunParamNode = InlinedFunctionDefinition->Prototype->Parameters[ParamIndex];
				FParameterDeclarator* FunParam = FunParamNode->AsParameterDeclarator();
				check(FunParam);

				FDeclaration* LocalDeclaration = OutputNodes.AllocNode<FDeclaration>(SourceInfo);

				LocalDeclaration->Identifier = FunParam->Identifier;
				LocalDeclaration->Semantic = FunParam->Semantic;
				LocalDeclaration->bIsArray = FunParam->bIsArray;
				LocalDeclaration->ArraySize = FunParam->ArraySize;

				const bool bExclusiveOutParam = FunParam->Type->Qualifier.bOut && !FunParam->Type->Qualifier.bIn;

				FExpression* InitializerExpression = ParamIndex < NumProvidedParameters
					? CallExpression->Expressions[ParamIndex]
					: LocalDeclaration->Initializer = FunParam->DefaultValue;

					if (!bExclusiveOutParam)
					{
						LocalDeclaration->Initializer = InitializerExpression;
					}

					FDeclaratorList* LocalDeclaratorList = OutputNodes.AllocNode<FDeclaratorList>(SourceInfo);
					LocalDeclaratorList->Type = OutputNodes.AllocNode<FFullySpecifiedType>(SourceInfo);
					LocalDeclaratorList->Type->Qualifier = FunParam->Type->Qualifier;
					// Clear in/out qualifiers when declaring the local
					LocalDeclaratorList->Type->Qualifier.bIn = 0;
					LocalDeclaratorList->Type->Qualifier.bOut = 0;
					LocalDeclaratorList->Type->Specifier = FunParam->Type->Specifier; // shallow clone

					LocalDeclaratorList->Declarations.Add(LocalDeclaration);

					InlinedBody->Statements.Add(LocalDeclaratorList);

					// Save the parameter description
					if (FunParam->Type->Qualifier.bOut)
					{
						FOutParameter OutParam;
						OutParam.Declarator = FunParam;
						OutParam.Expression = InitializerExpression;
						OutParameters.Add(OutParam);
					}
			}

			InlinedBody->Statements.Append(InlinedFunctionDefinition->Body->Statements);

			for (const FOutParameter& OutParam : OutParameters)
			{
				FExpression* RHS = OutputNodes.AllocNode<FExpression>(EOperators::Identifier, SourceInfo);
				RHS->Identifier = OutParam.Declarator->Identifier;
				FBinaryExpression* AssignmentExpression = OutputNodes.AllocNode<FBinaryExpression>(EOperators::Assign, OutParam.Expression, RHS, SourceInfo);
				FExpressionStatement* AssignmentStatement = OutputNodes.AllocNode<FExpressionStatement>(AssignmentExpression, SourceInfo);
				InlinedBody->Statements.Add(AssignmentStatement);
			}

			Node = InlinedBody;
		}
	});

	FASTWriter GeneratedCodeWriter(InOutSourceCode);

	OutputFunction->Write(GeneratedCodeWriter);

	InOutEntryPoint = OptimizedEntryPoint;

	return true;
}

#if WITH_LOW_LEVEL_TESTS

static void GenerateShaderCode(const CrossCompiler::TLinearArray<CrossCompiler::AST::FNode*>& ASTNodes, FString& OutResult)
{
	CrossCompiler::AST::FASTWriter Writer(OutResult);
	for (const CrossCompiler::AST::FNode* Node : ASTNodes)
	{
		Node->Write(Writer);
	}
}

static TArray<FString> GetShaderInputSemantics(const FString& ShaderSource, const FString& EntryPoint)
{
	TArray<FString> Result;

	CrossCompiler::FCompilerMessages CompilerMessages;
	TArray<UE::HlslParser::FScopedDeclarations> ScopedDeclarations;
	CrossCompiler::Parser::Parse(ShaderSource, TEXT("shader.hlsl"), CompilerMessages, ScopedDeclarations,
		[&Result, &EntryPoint](CrossCompiler::FLinearAllocator* Allocator, CrossCompiler::TLinearArray<CrossCompiler::AST::FNode*>& ASTNodes)
		{
			using namespace CrossCompiler::AST;

			struct FStructSemantic
			{
				const TCHAR* StructName = nullptr;
				const TCHAR* Semantic = nullptr;
			};

			TArray<FStructSemantic> KnownStructSemantics;

			for (FNode* Node : ASTNodes)
			{
				if (FFunctionDefinition* FunctionDefinition = Node->AsFunctionDefinition())
				{
					FFunction* Prototype = FunctionDefinition->Prototype;

					if (!Prototype || !Prototype->Identifier->Equals(EntryPoint))
					{
						continue;
					}

					for (FNode* ParamNode : Prototype->Parameters)
					{
						FParameterDeclarator* ParamDeclarator = ParamNode->AsParameterDeclarator();
						if (!ParamDeclarator)
						{
							continue;
						}

						if (ParamDeclarator->Semantic && ParamDeclarator->Semantic->Semantic)
						{
							Result.Add(FString(ParamDeclarator->Semantic->Semantic));
						}
						else if (ParamDeclarator->Type && ParamDeclarator->Type->Specifier && ParamDeclarator->Type->Specifier->TypeName)
						{
							const TCHAR* TypeName = ParamDeclarator->Type->Specifier->TypeName;
							for (const FStructSemantic& Semantic : KnownStructSemantics)
							{
								if (!FCString::Strcmp(Semantic.StructName, TypeName))
								{
									Result.Add(FString(Semantic.Semantic));
								}
							}
						}
					}
				}
				else if (FDeclaratorList* DeclaratorList = Node->AsDeclaratorList())
				{
					if (DeclaratorList->Type
						&& DeclaratorList->Type->Specifier
						&& DeclaratorList->Type->Specifier->Structure
						&& DeclaratorList->Type->Specifier->Structure->Name)
					{
						auto Struct = DeclaratorList->Type->Specifier->Structure;
						for (FNode* StructMember : Struct->Members)
						{
							FDeclaratorList* MemberDeclaratorList = StructMember->AsDeclaratorList();
							if (!MemberDeclaratorList)
							{
								continue;
							}

							for (FNode* MemberDeclarationNode : MemberDeclaratorList->Declarations)
							{
								FDeclaration* MemberDeclaration = MemberDeclarationNode->AsDeclaration();
								if (!MemberDeclaration)
								{
									continue;
								}

								if (MemberDeclaration->Semantic && MemberDeclaration->Semantic->Semantic)
								{
									FStructSemantic Entry;
									Entry.StructName = DeclaratorList->Type->Specifier->Structure->Name;
									Entry.Semantic = MemberDeclaration->Semantic->Semantic;
									KnownStructSemantics.Add(Entry);
								}
							}
						}
					}
				}
			}
		});

	return Result;
}

TEST_CASE_NAMED(FShaderRemoveUnusedInputsLooseParametersTest, "Shaders::RemoveUnusedInputs::LooseParameters", "[EditorContext][EngineFilter]")
{
	FString ShaderCodeString(
		TEXT(R"(
float4 main(float4 A : TEXCOORD0, float4 B : TEXCOORD1) : SV_Target
{
	return B;
}
)"));

	FString EntryPoint(TEXT("main"));

	TArray<FString> ParametersToKeep;
	ParametersToKeep.Add(TEXT("TEXCOORD1"));

	TArray<FString> Errors;
	const bool bSuccess = UE::HlslParser::RemoveUnusedInputs(ShaderCodeString, ParametersToKeep, EntryPoint, Errors);
	for (const FString& Message : Errors)
	{
		ADD_ERROR(Message);
	}
	TEST_TRUE(TEXT("RemoveUnusedInputs succeeded"), bSuccess);

	TArray<FString> RewrittenInputs = GetShaderInputSemantics(ShaderCodeString, EntryPoint);
	TEST_FALSE(TEXT("Rewritten shader uses TEXCOORD1"), RewrittenInputs.Contains(TEXT("TEXCOORD0")));
	TEST_TRUE(TEXT("Rewritten shader uses TEXCOORD1"), RewrittenInputs.Contains(TEXT("TEXCOORD1")));
}

TEST_CASE_NAMED(FShaderRemoveUnusedInputsStructParametersTest, "Shaders::RemoveUnusedInputs::StructParameters", "[EditorContext][EngineFilter]")
{
	FString ShaderCodeString(
		TEXT(R"(
struct FPSInputs
{
	float4 A : TEXCOORD0;
	float4 B : TEXCOORD1;
};
float4 main(FPSInputs Inputs) : SV_Target
{
	return Inputs.B;
}
)"));

	FString EntryPoint(TEXT("main"));

	TArray<FString> ParametersToKeep;
	ParametersToKeep.Add(TEXT("TEXCOORD1"));

	TArray<FString> Errors;
	const bool bSuccess = UE::HlslParser::RemoveUnusedInputs(ShaderCodeString, ParametersToKeep, EntryPoint, Errors);
	for (const FString& Message : Errors)
	{
		ADD_ERROR(Message);
	}
	TEST_TRUE(TEXT("RemoveUnusedInputs succeeded"), bSuccess);

	TArray<FString> RewrittenInputs = GetShaderInputSemantics(ShaderCodeString, EntryPoint);
	TEST_FALSE(TEXT("Rewritten shader uses TEXCOORD0"), RewrittenInputs.Contains(TEXT("TEXCOORD0")));
	TEST_TRUE(TEXT("Rewritten shader uses TEXCOORD1"), RewrittenInputs.Contains(TEXT("TEXCOORD1")));
}

TEST_CASE_NAMED(FShaderRemoveUnusedInputsMixedParametersTest, "Shaders::RemoveUnusedInputs::MixedParameters", "[EditorContext][EngineFilter]")
{
	FString ShaderCodeString(
		TEXT(R"(
struct FPSInputs
{
	float4 A : TEXCOORD0;
	float4 B : TEXCOORD1;
};
float4 main(FPSInputs Inputs, float4 C : TEXCOORD2) : SV_Target
{
	return Inputs.A + C;
}
)"));

	FString EntryPoint(TEXT("main"));

	TArray<FString> ParametersToKeep;
	ParametersToKeep.Add(TEXT("TEXCOORD1"));
	ParametersToKeep.Add(TEXT("TEXCOORD2"));

	TArray<FString> Errors;
	const bool bSuccess = UE::HlslParser::RemoveUnusedInputs(ShaderCodeString, ParametersToKeep, EntryPoint, Errors);
	for (const FString& Message : Errors)
	{
		ADD_ERROR(Message);
	}
	TEST_TRUE(TEXT("RemoveUnusedInputs succeeded"), bSuccess);

	TArray<FString> RewrittenInputs = GetShaderInputSemantics(ShaderCodeString, EntryPoint);
	TEST_FALSE(TEXT("Rewritten shader uses TEXCOORD0"), RewrittenInputs.Contains(TEXT("TEXCOORD0")));
	TEST_TRUE(TEXT("Rewritten shader uses TEXCOORD1"), RewrittenInputs.Contains(TEXT("TEXCOORD1")));
	TEST_TRUE(TEXT("Rewritten shader uses TEXCOORD2"), RewrittenInputs.Contains(TEXT("TEXCOORD2")));
}

TEST_CASE_NAMED(FShaderRenameLocalsTest, "Shaders::RenameLocals", "[EditorContext][EngineFilter]")
{
	FString ShaderCodeString(
		TEXT(R"(
float fun(float Param_A, float Param_B, float Param_C)
{
	float C = 123.0;
	return Param_B * Param_A + C * Param_C;
}
float4 main(float4 Param_C : TEXCOORD0) : SV_Target
{
	// test comment
	float A = 0.1;
	float B = 0.2 + A;
	for (int I=0; I < 4; ++I)
	{
		B *= 2.0;
	}
	for (int J=0, K=1; J < 4; ++J)
	{
		B *= 2.0;
		B += (float)K;
	}

	int X = int(Param_C.x);
	switch (X)
	{
	case 0:
		A *= 2.0;
		break;
	default:
		break;
	}

	return float4(A, B, A+B, fun(A,B)) + Param_C;
}
float4 bar(float Param_A, float Param_B, float4 Param_C)
{
	return Param_C * Param_A + Param_B;
}
)"));

	FString EntryPoint(TEXT("main"));

	FString Result;

	CrossCompiler::FCompilerMessages CompilerMessages;
	TArray<UE::HlslParser::FScopedDeclarations> ScopedDeclarations;
	CrossCompiler::Parser::Parse(ShaderCodeString, TEXT("shader.hlsl"), CompilerMessages, ScopedDeclarations,
		[&Result, &EntryPoint](CrossCompiler::FLinearAllocator* Allocator, CrossCompiler::TLinearArray<CrossCompiler::AST::FNode*>& ASTNodes)
		{
			using namespace CrossCompiler::AST;

			auto ProcessIdentifier = [Allocator](FIdentifier* Identifier)
			{
				TStringBuilder<256> NewName;
				NewName << TEXT("Renamed_");
				NewName << Identifier->ToStringView();

				Identifier->Rename(Allocator, NewName);
			};

			for (FNode* Node : ASTNodes)
			{
				FFunctionDefinition* FunctionDefinition = Node->AsFunctionDefinition();
				if (!FunctionDefinition || !FunctionDefinition->Body || !FunctionDefinition->Prototype->Identifier->Equals(EntryPoint))
				{
					continue;
				}

				VisitFunctionIdentifiers(FunctionDefinition, ProcessIdentifier);
			}

			GenerateShaderCode(ASTNodes, Result);
		});

	// Just a basic check at this point.
	// TODO: we could parse the rewritten shader and validate that old identifiers are not present and new ones are.

	TEST_TRUE(TEXT("Function signature uses renamed parameters"), Result.Contains(TEXT("float4 main(float4 Renamed_Param_C : TEXCOORD0 ) : SV_Target")));
	TEST_TRUE(TEXT("Function uses renamed for loop variable I"), Result.Contains(TEXT("for (int Renamed_I = 0; Renamed_I < 4; ++Renamed_I)")));
	TEST_TRUE(TEXT("Function uses renamed for loop variables J and K"), Result.Contains(TEXT("for (int Renamed_J = 0, Renamed_K = 1; Renamed_J < 4; ++Renamed_J)")));
	TEST_TRUE(TEXT("Function uses renamed locals"), Result.Contains(TEXT("float4(Renamed_A, Renamed_B, (Renamed_A + Renamed_B), fun(Renamed_A, Renamed_B)) + Renamed_Param_C")));
	TEST_TRUE(TEXT("Function foo() signature is expected to be unchanged"), Result.Contains(TEXT("fun(float Param_A, float Param_B, float Param_C)")));
	TEST_TRUE(TEXT("Function bar() signature is expected to be unchanged"), Result.Contains(TEXT("float4 bar(float Param_A, float Param_B, float4 Param_C)")));
}

TEST_CASE_NAMED(FShaderInlineFunctionTest, "Shaders::InlineFunction", "[EditorContext][EngineFilter]")
{
	FString ShaderCodeString(
		TEXT(R"(// dxc -T ps_6_0 inline.hlsl
struct FParams
{
	float a;
	float b;
};

void foo(out float4 res, in FParams params, float a, float b = 0.0, int q=1)
{
	if (b > 10.0)
	{
		a *= b;
	}

	if (b < -10.0)
		b *= 2;

	float c = (a + b + params.a) * params.b;
	c = c + 1.0;
	res = float4(a, b, c, 1.0);

	for (int i=0; i<10; ++i)
	{
		int j = i * 2;
		a += 0.1 + (float)j;
	}

	if (b < 0)
	{
		int x = 1;
		res.x += 1.0;
	}

	switch (q)
	{
		case 0:
			break;
		case 1:
		case 2:
		{
			res.x += 1.0;
			break;
		}
		case 3:
		{
			int w = q;
			res.x += (float)w;
			break;
		}
	}
}

float4 main() : SV_Target
{
	float x = 5.0;
	float c = 3.0;
	float4 res;
	FParams params;
	params.a = 1.5f;
	params.b = 2.5f;
	foo(res, params, 1.23f);
	res.x += x * c;
	return res;
})"));

	FString EntryPointString(TEXT("main"));
	FString FunctionNameString(TEXT("foo"));

	TArray<FString> Errors;
	const bool bSuccess = InlineFunction(ShaderCodeString, EntryPointString, FunctionNameString, Errors);

	TEST_TRUE(TEXT("Function inlining succeeded"), bSuccess);
	TEST_TRUE(TEXT("Function arguments are handled"), ShaderCodeString.Contains(TEXT("float4 INLINE_res;")));
	TEST_TRUE(TEXT("Function arguments with local parameters are handled"), ShaderCodeString.Contains(TEXT("FParams INLINE_params = params;")));
	TEST_TRUE(TEXT("Function arguments with local parameters are handled"), ShaderCodeString.Contains(TEXT("float INLINE_a = 1.23f;")));
	TEST_TRUE(TEXT("Function arguments with default values are handled"), ShaderCodeString.Contains(TEXT("float INLINE_b = 0.0;")));
	TEST_TRUE(TEXT("Function arguments with default values are handled"), ShaderCodeString.Contains(TEXT("int INLINE_q = 1;")));
	TEST_TRUE(TEXT("Out parameter is handled"), ShaderCodeString.Contains(TEXT("res = INLINE_res")));
}

#endif // WITH_LOW_LEVEL_TESTS
