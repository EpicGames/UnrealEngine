// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HlslUtils.h - Utilities for Hlsl.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

#define USE_UNREAL_ALLOCATOR	0
#define USE_PAGE_POOLING		0

namespace CrossCompiler
{
	namespace AST
	{
		class FNode;
		struct FFunctionDefinition;
		class FIdentifier;
	}

	namespace Memory
	{
		struct FPage
		{
			int8* Current;
			int8* Begin;
			int8* End;

			FPage(SIZE_T Size)
			{
				check(Size > 0);
				Begin = new int8[Size];
				End = Begin + Size;
				Current = Begin;
			}

			~FPage()
			{
				delete[] Begin;
			}

			static FPage* AllocatePage(SIZE_T PageSize);
			static void FreePage(FPage* Page);
		};

		enum
		{
			MinPageSize = 64 * 1024
		};
	}

	struct FLinearAllocator
	{
		UE_NONCOPYABLE(FLinearAllocator);

		FLinearAllocator()
		{
			auto* Initial = Memory::FPage::AllocatePage(Memory::MinPageSize);
			Pages.Add(Initial);
		}

		~FLinearAllocator()
		{
			for (auto* Page : Pages)
			{
				Memory::FPage::FreePage(Page);
			}
		}

		inline void* Alloc(SIZE_T NumBytes)
		{
			auto* Page = Pages.Last();
			if (Page->Current + NumBytes > Page->End)
			{
				SIZE_T PageSize = FMath::Max<SIZE_T>(Memory::MinPageSize, NumBytes);
				Page = Memory::FPage::AllocatePage(PageSize);
				Pages.Add(Page);
			}

			void* Ptr = Page->Current;
			Page->Current += NumBytes;
			return Ptr;
		}

		inline void* Alloc(SIZE_T NumBytes, SIZE_T Align)
		{
			void* Data = Alloc(NumBytes + Align - 1);
			UPTRINT Address = (UPTRINT)Data;
			Address += (Align - (Address % (UPTRINT)Align)) % Align;
			return (void*)Address;
		}

		TCHAR* Strdup(const TCHAR* String, int32 Length = -1)
		{
			if (!String)
			{
				return nullptr;
			}

			if (Length < 0)
			{
				Length = FCString::Strlen(String);
			}

			const auto Size = (Length + 1) * sizeof(TCHAR);
			auto* Data = (TCHAR*)Alloc(Size, sizeof(TCHAR));

			FCString::Strncpy(Data, String, Length + 1);

			return Data;
		}

		inline TCHAR* Strdup(const FStringView String)
		{
			return Strdup(String.GetData(), String.Len());
		}

		TArray<Memory::FPage*, TInlineAllocator<8> > Pages;
	};

#if !USE_UNREAL_ALLOCATOR
	class FLinearAllocatorPolicy
	{
	public:
		using SizeType = int32;

		// Unreal allocator magic
		enum { NeedsElementType = false };
		enum { RequireRangeCheck = true };

		template<typename ElementType>
		class ForElementType
		{
		public:

			/** Default constructor. */
			ForElementType() :
				LinearAllocator(nullptr),
				Data(nullptr)
			{}

			// FContainerAllocatorInterface
			/*FORCEINLINE*/ ElementType* GetAllocation() const
			{
				return Data;
			}
			void ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement)
			{
				void* OldData = Data;
				if (NewMax)
				{
					// Allocate memory from the stack.
					Data = (ElementType*)LinearAllocator->Alloc(NewMax * NumBytesPerElement,
						FMath::Max((uint32)sizeof(void*), (uint32)alignof(ElementType))
						);

					// If the container previously held elements, copy them into the new allocation.
					if (OldData && CurrentNum)
					{
						const SizeType NumCopiedElements = FMath::Min(NewMax, CurrentNum);
						FMemory::Memcpy(Data, OldData, NumCopiedElements * NumBytesPerElement);
					}
				}
			}
			SizeType CalculateSlackReserve(SizeType NewMax, SIZE_T NumBytesPerElement) const
			{
				return DefaultCalculateSlackReserve(NewMax, NumBytesPerElement, false);
			}
			SizeType CalculateSlackShrink(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
			{
				return DefaultCalculateSlackShrink(NewMax, CurrentMax, NumBytesPerElement, false);
			}
			SizeType CalculateSlackGrow(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
			{
				return DefaultCalculateSlackGrow(NewMax, CurrentMax, NumBytesPerElement, false);
			}

			SIZE_T GetAllocatedSize(SizeType CurrentMax, SIZE_T NumBytesPerElement) const
			{
				return CurrentMax * NumBytesPerElement;
			}

			bool HasAllocation() const
			{
				return !!Data;
			}

			SizeType GetInitialCapacity() const
			{
				return 0;
			}

			FLinearAllocator* LinearAllocator;

		private:

			/** A pointer to the container's elements. */
			ElementType* Data;
		};

		typedef ForElementType<FScriptContainerElement> ForAnyElementType;
	};

	class FLinearBitArrayAllocator
		: public TInlineAllocator<4, FLinearAllocatorPolicy>
	{
	};

	class FLinearSparseArrayAllocator
		: public TSparseArrayAllocator<FLinearAllocatorPolicy, FLinearBitArrayAllocator>
	{
	};

	class FLinearSetAllocator
		: public TSetAllocator<FLinearSparseArrayAllocator, TInlineAllocator<1, FLinearAllocatorPolicy> >
	{
	};

	template <typename TType>
	class TLinearArray : public TArray<TType, FLinearAllocatorPolicy>
	{
	public:
		TLinearArray(FLinearAllocator* Allocator)
		{
			TArray<TType, FLinearAllocatorPolicy>::AllocatorInstance.LinearAllocator = Allocator;
		}
	};

	struct FNodeContainer
	{
		UE_NONCOPYABLE(FNodeContainer);

		FNodeContainer()
			: Nodes(&Allocator)
		{
		}

		template <typename T, typename... ArgsType>
		T* AllocNode(ArgsType&&... Args)
		{
			static_assert(std::is_base_of_v<CrossCompiler::AST::FNode, T>, "Type must inherit from CrossCompiler::AST::FNode");
			T* Result = new(&Allocator) T(&Allocator, Forward<ArgsType>(Args)...);
			return Result;
		}

		FLinearAllocator Allocator;
		TLinearArray<CrossCompiler::AST::FNode*> Nodes;
	};

	/*
	template <typename TType>
	struct TLinearSet : public TSet<typename TType, DefaultKeyFuncs<typename TType>, FLinearSetAllocator>
	{
	TLinearSet(FLinearAllocator* InAllocator)
	{
	Elements.AllocatorInstance.LinearAllocator = InAllocator;
	}
	};*/
#endif

	struct FSourceInfo
	{
		FString* Filename;
		int32 Line;
		int32 Column;

		FSourceInfo() : Filename(nullptr), Line(0), Column(0) {}
	};

	struct FCompilerMessages
	{
		struct FMessage
		{
			//FSourceInfo SourceInfo;
			bool bIsError;
			FString Message;

			FMessage(bool bInIsError, const FString& InMessage) :
				bIsError(bInIsError),
				Message(InMessage)
			{
			}
		};
		TArray<FMessage> MessageList;

		inline void AddMessage(bool bIsError, const FString& Message)
		{
			auto& NewMessage = MessageList.Emplace_GetRef(bIsError, Message);
		}

		inline void SourceError(const FSourceInfo& SourceInfo, const TCHAR* String)
		{
			if (SourceInfo.Filename)
			{
				AddMessage(true, FString::Printf(TEXT("%s(%d,%d): %s\n"), **SourceInfo.Filename, SourceInfo.Line, SourceInfo.Column, String));
			}
			else
			{
				AddMessage(true, FString::Printf(TEXT("<unknown>(%d,%d): %s\n"), SourceInfo.Line, SourceInfo.Column, String));
			}
		}

		inline void SourceError(const TCHAR* String)
		{
			AddMessage(true, FString::Printf(TEXT("%s\n"), String));
		}

		inline void SourceWarning(const FSourceInfo& SourceInfo, const TCHAR* String)
		{
			if (SourceInfo.Filename)
			{
				AddMessage(false, FString::Printf(TEXT("%s(%d,%d): %s\n"), **SourceInfo.Filename, SourceInfo.Line, SourceInfo.Column, String));
			}
			else
			{
				AddMessage(false, FString::Printf(TEXT("<unknown>(%d,%d): %s\n"), SourceInfo.Line, SourceInfo.Column, String));
			}
		}

		inline void SourceWarning(const TCHAR* String)
		{
			AddMessage(false, FString::Printf(TEXT("%s\n"), String));
		}
	};
}
